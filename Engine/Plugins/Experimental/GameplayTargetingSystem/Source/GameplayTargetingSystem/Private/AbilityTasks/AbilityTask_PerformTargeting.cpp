// Copyright Epic Games, Inc. All Rights Reserved.
#include "AbilityTasks/AbilityTask_PerformTargeting.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Types/TargetingSystemLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbilityTask_PerformTargeting)

void UAbilityTask_PerformTargeting::Activate()
{
	if (AActor* SourceActor = GetAvatarActor())
	{
		if (UTargetingSubsystem* TargetingSubsystem = UTargetingSubsystem::Get(SourceActor->GetWorld()))
		{
			FTargetingSourceContext SourceContext;
			SourceContext.SourceActor = SourceActor;

			FTargetingRequestHandle TargetingHandle = UTargetingSubsystem::MakeTargetRequestHandle(TargetingPreset, SourceContext);
			
			SetupInitialTargetsForRequest(TargetingHandle);
			
			FTargetingRequestDelegate Delegate = FTargetingRequestDelegate::CreateWeakLambda(this, [this](FTargetingRequestHandle TargetingHandle)
			{
				OnTargetReady.Broadcast(TargetingHandle);
			});

			if (bPerformAsync)
			{
				TARGETING_LOG(Verbose, TEXT("Starting async targeting ability task"));

				FTargetingAsyncTaskData& AsyncTaskData = FTargetingAsyncTaskData::FindOrAdd(TargetingHandle);
				AsyncTaskData.bReleaseOnCompletion = true;

				TargetingSubsystem->StartAsyncTargetingRequestWithHandle(TargetingHandle, Delegate);
			}
			else
			{
				TARGETING_LOG(Verbose, TEXT("Starting immediate targeting ability task"));

				FTargetingImmediateTaskData& ImmeidateTaskData = FTargetingImmediateTaskData::FindOrAdd(TargetingHandle);
				ImmeidateTaskData.bReleaseOnCompletion = true;

				TargetingSubsystem->ExecuteTargetingRequestWithHandle(TargetingHandle, Delegate);

				EndTask();
			}
		}
	}
	else
	{
		TARGETING_LOG(Error, TEXT("%s called with an invalid source actor!"), ANSI_TO_TCHAR(__FUNCTION__));
		EndTask();
	}
}

UAbilityTask_PerformTargeting* UAbilityTask_PerformTargeting::PerformTargetingRequest(UGameplayAbility* OwningAbility, UTargetingPreset* InTargetingPreset, bool bAllowAsync)
{
	if (!InTargetingPreset)
	{
		return nullptr;
	}

	UAbilityTask_PerformTargeting* Task = NewAbilityTask<UAbilityTask_PerformTargeting>(OwningAbility);
	Task->TargetingPreset = InTargetingPreset;
	Task->bPerformAsync = bAllowAsync;

	return Task;
}

UAbilityTask_PerformTargeting* UAbilityTask_PerformTargeting::PerformFilteringRequest(UGameplayAbility* OwningAbility, UTargetingPreset* InTargetingPreset, const TArray<AActor*> InTargets, bool bAllowAsync)
{
	if (!InTargetingPreset)
	{
		return nullptr;
	}

	UAbilityTask_PerformTargeting* Task = NewAbilityTask<UAbilityTask_PerformTargeting>(OwningAbility);
	Task->TargetingPreset = InTargetingPreset;
	Task->InitialTargets = InTargets;
	Task->bPerformAsync = bAllowAsync;

	return Task;
}

void UAbilityTask_PerformTargeting::SetupInitialTargetsForRequest(FTargetingRequestHandle RequestHandle) const
{
	if (RequestHandle.IsValid() && InitialTargets.Num() > 0)
	{
		FTargetingDefaultResultsSet& TargetingResults = FTargetingDefaultResultsSet::FindOrAdd(RequestHandle);
		for (AActor* Target : InitialTargets)
		{
			if (!Target)
			{
				continue;
			}

			bool bAddResult = !TargetingResults.TargetResults.ContainsByPredicate([Target](const FTargetingDefaultResultData& Data) -> bool
			{
				return (Data.HitResult.GetActor() == Target);
			});

			if (bAddResult)
			{
				FTargetingDefaultResultData* ResultData = new(TargetingResults.TargetResults) FTargetingDefaultResultData();
				ResultData->HitResult.HitObjectHandle = FActorInstanceHandle(Target);
				ResultData->HitResult.Location = Target->GetActorLocation();
			}
		}

	}
}
