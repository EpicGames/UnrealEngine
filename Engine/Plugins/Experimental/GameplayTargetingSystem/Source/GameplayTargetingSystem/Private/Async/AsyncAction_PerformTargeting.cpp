// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/AsyncAction_PerformTargeting.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "TargetingSystem/TargetingPreset.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Types/TargetingSystemLogs.h"
#include "Types/TargetingSystemTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncAction_PerformTargeting)


UAsyncAction_PerformTargeting::UAsyncAction_PerformTargeting(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bUseAsyncTargeting = false;
}

UAsyncAction_PerformTargeting* UAsyncAction_PerformTargeting::PerformTargetingRequest(AActor* SourceActor, UTargetingPreset* TargetingPreset, bool bUseAsyncTargeting)
{
	if (!TargetingPreset)
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(SourceActor, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return nullptr;
	}

	UAsyncAction_PerformTargeting* Action = NewObject<UAsyncAction_PerformTargeting>();
	Action->TargetingPreset = TargetingPreset;
	Action->WeakSourceActor = SourceActor;
	Action->bUseAsyncTargeting = bUseAsyncTargeting;
	Action->RegisterWithGameInstance(World);

	return Action;
}

UAsyncAction_PerformTargeting* UAsyncAction_PerformTargeting::PerformFilteringRequest(AActor* SourceActor, UTargetingPreset* TargetingPreset, bool bUseAsyncTargeting, const TArray<AActor*> InTargets)
{
	if (!TargetingPreset)
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(SourceActor, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return nullptr;
	}

	UAsyncAction_PerformTargeting* Action = NewObject<UAsyncAction_PerformTargeting>();
	Action->TargetingPreset = TargetingPreset;
	Action->WeakSourceActor = SourceActor;
	Action->bUseAsyncTargeting = bUseAsyncTargeting;
	Action->InitialTargets = InTargets;
	Action->RegisterWithGameInstance(World);

	return Action;
}

void UAsyncAction_PerformTargeting::Activate()
{
	if (AActor* SourceActor = WeakSourceActor.Get())
	{
		if (UTargetingSubsystem* TargetingSubsystem = UTargetingSubsystem::Get(SourceActor->GetWorld()))
		{
			FTargetingSourceContext SourceContext;
			SourceContext.SourceActor = SourceActor;

			TARGETING_LOG(Verbose, TEXT("Source Actor: %s"), *GetNameSafe(SourceContext.SourceActor));

			TargetingHandle = UTargetingSubsystem::MakeTargetRequestHandle(TargetingPreset, SourceContext);

			SetupInitialTargetsForRequest();

			FTargetingRequestDelegate Delegate = FTargetingRequestDelegate::CreateWeakLambda(this, [this](FTargetingRequestHandle InTargetingHandle)
				{
					TARGETING_LOG(Verbose, TEXT("Entering request lambda"));

					Targeted.Broadcast(InTargetingHandle);
					SetReadyToDestroy();
				});

			if (bUseAsyncTargeting)
			{
				TARGETING_LOG(Verbose, TEXT("Starting async targeting"));

				FTargetingAsyncTaskData& AsyncTaskData = FTargetingAsyncTaskData::FindOrAdd(TargetingHandle);
				AsyncTaskData.bReleaseOnCompletion = true;

				TargetingSubsystem->StartAsyncTargetingRequestWithHandle(TargetingHandle, Delegate);
			}
			else
			{
				TARGETING_LOG(Verbose, TEXT("Starting immediate targeting"));

				FTargetingImmediateTaskData& ImmediateTaskData = FTargetingImmediateTaskData::FindOrAdd(TargetingHandle);
				ImmediateTaskData.bReleaseOnCompletion = true;

				TargetingSubsystem->ExecuteTargetingRequestWithHandle(TargetingHandle, Delegate);
			}
		}
	}
	else
	{
		SetReadyToDestroy();
	}
}

void UAsyncAction_PerformTargeting::SetupInitialTargetsForRequest() const
{
	if (TargetingHandle.IsValid() && InitialTargets.Num() > 0)
	{
		FTargetingDefaultResultsSet& TargetingResults = FTargetingDefaultResultsSet::FindOrAdd(TargetingHandle);
		for (AActor* Target : InitialTargets)
		{
			if (!Target)
			{
				continue;
			}

			bool bAddResult = !TargetingResults.TargetResults.ContainsByPredicate([Target](const FTargetingDefaultResultData& Data) -> bool
				{
					return (Data.HitResult.GetActor() == Target);
				});

			if (bAddResult)
			{
				FTargetingDefaultResultData* ResultData = new(TargetingResults.TargetResults) FTargetingDefaultResultData();
				ResultData->HitResult.HitObjectHandle = FActorInstanceHandle(Target);
				ResultData->HitResult.Location = Target->GetActorLocation();
			}
		}

	}
}
