// Copyright Epic Games, Inc. All Rights Reserved.
#include "TargetingSystem/TargetingPreset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingPreset)


UTargetingPreset::UTargetingPreset(const FObjectInitializer& ObjectInitializer)
{

}

