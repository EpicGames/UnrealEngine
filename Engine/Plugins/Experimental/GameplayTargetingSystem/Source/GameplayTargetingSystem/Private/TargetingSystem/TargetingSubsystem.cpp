// Copyright Epic Games, Inc. All Rights Reserved.
#include "TargetingSystem/TargetingSubsystem.h"

#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Kismet/KismetStringLibrary.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#include "TargetingSystem/TargetingPreset.h"
#include "Tasks/CollisionQueryTaskData.h"
#include "Types/TargetingSystemLogs.h"
#include "Types/TargetingSystemTypes.h"
#include "Tasks/TargetingTask.h"

#if ENABLE_DRAW_DEBUG
#include "GameFramework/HUD.h"
#include "Engine/Canvas.h"
#include "DisplayDebugHelpers.h"
#endif // ENABLE_DRAW_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingSubsystem)

namespace TargetingSystemCVars
{
	static bool bUseAsyncTargetingTimeSlicing = true;
	FAutoConsoleVariableRef CvarUseAsyncTargetingTimeSlicing(
		TEXT("ts.UseAsyncTargetingTimeSlicing"),
		bUseAsyncTargetingTimeSlicing,
		TEXT("Toggles whether the targeting system will use time slicing for the async targeting request queue. (Enabled: true, Disabled: false)")
	);

	static float MaxAsyncTickTime = .01f;
	FAutoConsoleVariableRef CvarMaxAsyncTickTime(
		TEXT("ts.MaxAsyncTickTime"),
		MaxAsyncTickTime,
		TEXT("Sets the total number of seconds we will allow to process the async targeting request queue.")
	);

#if ENABLE_DRAW_DEBUG

	static bool bEnableTargetingDebugging = false;
	FAutoConsoleVariableRef CvarEnableTargetingDebugging(
		TEXT("ts.debug.EnableTargetingDebugging"),
		bEnableTargetingDebugging,
		TEXT("Toggles whether the targeting system is actively in debugging mode. (Enabled: true, Disabled: false)")
	);

	static bool bForceRequeueAsyncRequests = false;
	FAutoConsoleVariableRef CvarForceRequeueAsyncRequests(
		TEXT("ts.debug.ForceRequeueAsyncRequests"),
		bForceRequeueAsyncRequests,
		TEXT("Toggles whether the targeting system will force requeue async requests. (Enabled: true, Disabled: false)")
	);

	static bool bPrintTargetingDebugToLog = false;
	FAutoConsoleVariableRef CvarPrintTargetingDebugToLog(
		TEXT("ts.debug.PrintTargetingDebugToLog"),
		bPrintTargetingDebugToLog,
		TEXT("Toggles we print the targeting debug text to the log. (Enabled: true, Disabled: false)")
	);

	static int32 TotalDebugRecentRequestsTracked = 5;
	FAutoConsoleVariableRef CvarTotalDebugRecentRequestsTracked(
		TEXT("ts.debug.TotalDebugRecentRequestsTracked"),
		TotalDebugRecentRequestsTracked,
		TEXT("Sets the total # of targeting requests that will be tracked upon starting of them (default = 5)")
	);
	
	static float OverrideTargetingLifeTime = 0.f;
    FAutoConsoleVariableRef CvarOverrideTargetingLifeTime(
    	TEXT("ts.debug.OverrideTargetingLifeTime"),
    	OverrideTargetingLifeTime,
    	TEXT("Overrides the draws life time to ease the debugging")
    );

#endif // ENABLE_DRAW_DEBUG
}

UTargetingSubsystem::UTargetingSubsystem()
: UGameInstanceSubsystem()
{
}

/* static */
UTargetingSubsystem* UTargetingSubsystem::Get(const UWorld* World)
{
	if (World)
	{
		return UGameInstance::GetSubsystem<UTargetingSubsystem>(World->GetGameInstance());
	}

	return nullptr;
}

/* static */
UTargetingSubsystem* UTargetingSubsystem::GetTargetingSubsystem(const UObject* WorldContextObject)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return UTargetingSubsystem::Get(World);
	}

	return nullptr;
}

void UTargetingSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UTargetingSubsystem* TypedThis = Cast<UTargetingSubsystem>(InThis);
	check(TypedThis);

	// Make sure any object references that async targeting requests are relying on are emitted
	for (FTargetingRequestHandle& CurHandle : TypedThis->AsyncTargetingRequests)
	{
		if (FTargetingSourceContext* FoundSource = FTargetingSourceContext::Find(CurHandle))
		{
			Collector.AddReferencedObject(FoundSource->InstigatorActor);
			Collector.AddReferencedObject(FoundSource->SourceActor);
			Collector.AddReferencedObject(FoundSource->SourceObject);
		}

		if (FTargetingTaskSet** FoundTaskSetPtr = const_cast<FTargetingTaskSet**>(FTargetingTaskSet::Find(CurHandle)))
		{
			if (FTargetingTaskSet* FoundTaskSet = (*FoundTaskSetPtr))
			{
				Collector.AddReferencedObjects<UTargetingTask>(FoundTaskSet->Tasks);
			}
		}
		
		if (FCollisionQueryTaskData* FoundOverride = UE::TargetingSystem::TTargetingDataStore<FCollisionQueryTaskData>::Find(CurHandle))
		{
			FoundOverride->AddStructReferencedObjects(Collector);
		}
	}
}

void UTargetingSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FCoreUObjectDelegates::PreLoadMap.AddUObject(this, &UTargetingSubsystem::HandlePreLoadMap);
}

void UTargetingSubsystem::Deinitialize()
{
	ClearAsyncRequests();
	FCoreUObjectDelegates::PreLoadMap.RemoveAll(this);

	// We are done, so do not attempt to tick us again
	SetTickableTickType(ETickableTickType::Never);
}

bool UTargetingSubsystem::Exec_Runtime(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (IsTemplate())
	{
		return false;
	}

#if ENABLE_DRAW_DEBUG
	if (FParse::Command(&Cmd, TEXT("ts.debug.ClearTrackedTargetRequests")))
	{
		CurrentImmediateRequestIndex = 0;
		for (FTargetingRequestHandle Handle : DebugTrackedImmediateTargetRequests)
		{
			ReleaseTargetRequestHandle(Handle);
		}

		DebugTrackedImmediateTargetRequests.Empty();

		for (FTargetingRequestHandle Handle : DebugTrackedAsyncTargetRequests)
		{
			ReleaseTargetRequestHandle(Handle);
		}

		DebugTrackedAsyncTargetRequests.Empty();

		return true;
	}
#endif // ENABLE_DRAW_DEBUG

	return false;
}

void UTargetingSubsystem::Tick(float DeltaTime)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(TargetingSystem);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UTargetingSubsystem::Tick"), STAT_TargetingSystem_Tick, STATGROUP_TargetingSystem);

	// @note: Might need to implement a scheme to prevent bottlenecking on a single request
	// or getting stuck from constant queuing of requests

	const int32 NumRequests = AsyncTargetingRequests.Num();
	{
		bTickingAsycnRequests = true;
		TARGETING_LOG(Verbose, TEXT("UTargetingSubsystem::Tick - Starting to Process %d requests"), NumRequests);

		float TimeLeft = TargetingSystemCVars::MaxAsyncTickTime;
		for (int32 RequestIterator = 0; RequestIterator < NumRequests; ++RequestIterator)
		{
			const double StepStartTime = FPlatformTime::Seconds();

			FTargetingRequestHandle& TargetingHandle = AsyncTargetingRequests[RequestIterator];
			TARGETING_LOG(Verbose, TEXT("UTargetingSubsystem::Tick - Started Processing Async Request [%d] with Handle [%d]"), RequestIterator, TargetingHandle.Handle);

			ProcessTargetingRequestTasks(TargetingHandle, TimeLeft);

			if (TargetingSystemCVars::bUseAsyncTargetingTimeSlicing)
			{
				const double StepDuration = (FPlatformTime::Seconds() - StepStartTime);
				TARGETING_LOG(VeryVerbose, TEXT("UTargetingSubsystem::Tick - Finished Processing Async Request [%d] in [%f] seconds."), RequestIterator, StepDuration);

				TimeLeft -= StepDuration;
				if (TimeLeft <= 0.0f)
				{
					TARGETING_LOG(VeryVerbose, TEXT("UTargetingSubsystem::Tick - Over alloted time, skipping remaining targeting requets this frame."));
					break;
				}
			}
		}

		bTickingAsycnRequests = false;
		OnFinishedTickingAsyncRequests();
	}
	
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UTargetingSubsystem::Tick_HandleCleanup"), STAT_TargetingSystem_TickHandleCleanup, STATGROUP_TargetingSystem);

		for (int32 RequestIterator = NumRequests - 1; RequestIterator >= 0; --RequestIterator)
		{
			FTargetingRequestHandle TargetingHandle = AsyncTargetingRequests[RequestIterator];
			if (FTargetingRequestData* RequestData = FTargetingRequestData::Find(TargetingHandle))
			{
				if (RequestData->bComplete)
				{
					AsyncTargetingRequests.RemoveAt(RequestIterator, 1, EAllowShrinking::No);

					bool bForceRequeueOnCompletion = false;
#if ENABLE_DRAW_DEBUG
					bForceRequeueOnCompletion = IsTargetingDebugEnabled() && TargetingSystemCVars::bForceRequeueAsyncRequests && DebugTrackedAsyncTargetRequests.Contains(TargetingHandle);
#endif // ENABLE_DRAW_DEBUG

					if (FTargetingAsyncTaskData* AsyncTaskData = FTargetingAsyncTaskData::Find(TargetingHandle))
					{
						// re-queue if setup to do so
						if (AsyncTaskData->bRequeueOnCompletion || bForceRequeueOnCompletion)
						{
							RequestData->bComplete = false;
							AsyncTaskData->InitializeForAsyncProcessing();

							if (FTargetingDefaultResultsSet* ResultSet = FTargetingDefaultResultsSet::Find(TargetingHandle))
							{
								ResultSet->TargetResults.Empty();
							}

							// put the handle back into the array at the end
							AsyncTargetingRequests.Add(TargetingHandle);
						}
						// release the handle if setup to do so
						else if (AsyncTaskData->bReleaseOnCompletion)
						{
							UTargetingSubsystem::ReleaseTargetRequestHandle(TargetingHandle);
						}
					}
					else
					{
						TARGETING_LOG(Error, TEXT("[%s] - No Async Task Data for Handle [%d]"), ANSI_TO_TCHAR(__FUNCTION__), TargetingHandle.Handle)

						UTargetingSubsystem::ReleaseTargetRequestHandle(TargetingHandle);
					}
				}
			}
		}
	}
}

void UTargetingSubsystem::OnFinishedTickingAsyncRequests()
{
	// Queue all the pending requests once we're safe to add to the Targeting Requests Data Store
	for (const TPair<FTargetingRequestHandle, FTargetingRequestData>& PendingRequest : PendingTargetingRequests)
	{
		ExecuteTargetingRequestWithHandle(PendingRequest.Key, PendingRequest.Value.TargetingRequestDelegate);
	}
	PendingTargetingRequests.Reset();

	for (const TPair<FTargetingRequestHandle, FTargetingRequestData>& PendingAsyncRequest : PendingAsyncTargetingRequests)
	{
		StartAsyncTargetingRequestWithHandle(PendingAsyncRequest.Key, PendingAsyncRequest.Value.TargetingRequestDelegate, PendingAsyncRequest.Value.TargetingRequestDynamicDelegate);
	}
	PendingAsyncTargetingRequests.Reset();
}

bool UTargetingSubsystem::IsTickable() const
{
	return !HasAnyFlags(RF_BeginDestroyed) && IsValidChecked(this) && (AsyncTargetingRequests.Num() > 0);
}

/* static */
FTargetingRequestHandle UTargetingSubsystem::CreateTargetRequestHandle()
{
	static int32 HandleGenerator = 0;
	FTargetingRequestHandle Handle(++HandleGenerator);
	return Handle;
}

/* static */
FTargetingRequestHandle UTargetingSubsystem::MakeTargetRequestHandle(const UTargetingPreset* TargetingPreset, const FTargetingSourceContext& InSourceContext)
{
	FTargetingRequestHandle Handle = CreateTargetRequestHandle();

	// store the task set
	if (TargetingPreset)
	{
		const FTargetingTaskSet*& TaskSet = FTargetingTaskSet::FindOrAdd(Handle);
		TaskSet = TargetingPreset->GetTargetingTaskSet();

		TARGETING_LOG(Verbose, TEXT("UTargetingSubsystem::MakeTargetRequestHandle - [%s] is Adding TaskSet [0x%08" UPTRINT_x_FMT "] for Handle [%d]"), *GetNameSafe(TargetingPreset), TaskSet, Handle.Handle);
	}
	else
	{
		TARGETING_LOG(Warning, TEXT("UTargetingSubsystem::MakeTargetRequestHandle - NULL TargetingPreset, no tasks will be setup for the targeting handle [%d]"), Handle.Handle);
	}

	// store the source context
	FTargetingSourceContext& SourceContext = FTargetingSourceContext::FindOrAdd(Handle);
	SourceContext = InSourceContext;

	return Handle;
}

/* static */
void UTargetingSubsystem::ReleaseTargetRequestHandle(FTargetingRequestHandle& Handle)
{
	// Caching the handle to be able to broadcast it later after the reset
	const FTargetingRequestHandle CachedHandle = Handle;
	Handle.Reset();
	
#if ENABLE_DRAW_DEBUG
	if (IsTargetingDebugEnabled())
	{
		return;
	}
#endif // ENABLE_DRAW_DEBUG

	FTargetingRequestHandle::GetReleaseHandleDelegate().Broadcast(CachedHandle);
	TARGETING_LOG(Verbose, TEXT("%s: - Releasigng Handle [%d]"), ANSI_TO_TCHAR(__FUNCTION__), CachedHandle.Handle);
}

void UTargetingSubsystem::ExecuteTargetingRequestWithHandle(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate)
{
	// If we're processing Targeting Requests, queue this one to prevent memory stomps. We'll add it once we finish Ticking Async Requests.
	if (bTickingAsycnRequests)
	{
		FTargetingRequestData& RequestData = PendingTargetingRequests.FindOrAdd(TargetingHandle);
		RequestData.Initialize(CompletionDelegate, CompletionDynamicDelegate, this);
		return;
	}

	ExecuteTargetingRequestWithHandleInternal(TargetingHandle, CompletionDelegate, CompletionDynamicDelegate);

	// if the caller setup task data to release on completion, lets do it
	if (const FTargetingImmediateTaskData* ImmediateTaskData = FTargetingImmediateTaskData::Find(TargetingHandle))
	{
		if (ImmediateTaskData->bReleaseOnCompletion)
		{
			UTargetingSubsystem::ReleaseTargetRequestHandle(TargetingHandle);
		}
	}
}

void UTargetingSubsystem::ExecuteTargetingRequest(const UTargetingPreset* TargetingPreset, const FTargetingSourceContext& SourceContext, FTargetingRequestDynamicDelegate CompletionDynamicDelegate)
{
	FTargetingRequestHandle RequestHandle;
	if (ensure(TargetingPreset))
	{
		RequestHandle = MakeTargetRequestHandle(TargetingPreset, SourceContext);

#if ENABLE_DRAW_DEBUG
#if WITH_EDITORONLY_DATA
		if (IsTargetingDebugEnabled())
		{
			FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(RequestHandle);
			DebugData.TargetingPreset = TargetingPreset;
		}
#endif // WITH_EDITORONLY_DATA
#endif // ENABLE_DRAW_DEBUG

		// If we're processing Targeting Requests, queue this one to prevent memory stomps. We'll add it once we finish Ticking Async Requests.
		FTargetingRequestDelegate CompletionDelegate;
		if (bTickingAsycnRequests)
		{
			FTargetingRequestData& RequestData = PendingTargetingRequests.FindOrAdd(RequestHandle);
			RequestData.Initialize(CompletionDelegate, CompletionDynamicDelegate, this);
			return;
		}

		ExecuteTargetingRequestWithHandleInternal(RequestHandle, CompletionDelegate, CompletionDynamicDelegate);
		UTargetingSubsystem::ReleaseTargetRequestHandle(RequestHandle);
	}
}

void UTargetingSubsystem::StartAsyncTargetingRequestWithHandle(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate)
{
	// If we're processing Targeting Requests, queue this one to prevent memory stomps. We'll add it once we finish Ticking Async Requests.
	if (bTickingAsycnRequests)
	{
		FTargetingRequestData& RequestData = PendingAsyncTargetingRequests.FindOrAdd(TargetingHandle);
		RequestData.Initialize(CompletionDelegate, CompletionDynamicDelegate, this);
		return;
	}

	StartAsyncTargetingRequestWithHandleInternal(TargetingHandle, CompletionDelegate, CompletionDynamicDelegate);
}

void UTargetingSubsystem::RemoveAsyncTargetingRequestWithHandle(FTargetingRequestHandle& TargetingHandle)
{
	if (TargetingHandle.IsValid())
	{
		// this is possible if remove is called during the completion callback
		bool bRequestComplete = false;
		if (FTargetingRequestData* RequestData = FTargetingRequestData::Find(TargetingHandle))
		{
			bRequestComplete = RequestData->bComplete;
		}
		
		if (!bRequestComplete)
		{
			if (UTargetingTask* ExecutingTask = FindCurrentExecutingTask(TargetingHandle))
			{
				ExecutingTask->CancelAsync();
			}
		}

		// if we are removing while processing the tick, we don't want to break the array, so defer cleanup
		if (bTickingAsycnRequests)
		{
			// flag for complete, so it cleans up when process is done
			if (FTargetingRequestData* RequestData = FTargetingRequestData::Find(TargetingHandle))
			{
				RequestData->bComplete = true;
			}
			
			// since it is removed, ensure requeue is disabled
			if (FTargetingAsyncTaskData* AsyncTaskData = FTargetingAsyncTaskData::Find(TargetingHandle))
			{
				AsyncTaskData->bRequeueOnCompletion = false;
			}

			// reset the handle for the caller, since they won't need it anymore
			TargetingHandle.Reset();
			return;
		}

		AsyncTargetingRequests.RemoveAll([TargetingHandle](const FTargetingRequestHandle& Handle)
		{
			return (TargetingHandle == Handle);
		});

		UTargetingSubsystem::ReleaseTargetRequestHandle(TargetingHandle);
	}
}

FTargetingRequestHandle UTargetingSubsystem::StartAsyncTargetingRequest(const UTargetingPreset* TargetingPreset, const FTargetingSourceContext& SourceContext, FTargetingRequestDynamicDelegate CompletionDynamicDelegate)
{
	FTargetingRequestHandle RequestHandle;
	if (ensure(TargetingPreset))
	{
		RequestHandle = MakeTargetRequestHandle(TargetingPreset, SourceContext);

#if ENABLE_DRAW_DEBUG
#if WITH_EDITORONLY_DATA
		if (IsTargetingDebugEnabled())
		{
			FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(RequestHandle);
			DebugData.TargetingPreset = TargetingPreset;
		}
#endif // WITH_EDITORONLY_DATA
#endif // ENABLE_DRAW_DEBUG

		FTargetingRequestDelegate CompletionDelegate;

		// If we're processing Targeting Requests, queue this one to prevent memory stomps. We'll add it once we finish Ticking Async Requests.
		if (bTickingAsycnRequests)
		{
			FTargetingRequestData& RequestData = PendingAsyncTargetingRequests.FindOrAdd(RequestHandle);
			RequestData.Initialize(CompletionDelegate, CompletionDynamicDelegate, this);
			return RequestHandle;
		}

		StartAsyncTargetingRequestWithHandleInternal(RequestHandle, CompletionDelegate, CompletionDynamicDelegate);
	}
	
	return RequestHandle;
}

void UTargetingSubsystem::ExecuteTargetingRequestWithHandleInternal(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate)
{
	if (ensure(TargetingHandle.IsValid()))
	{
		// store the request data
		FTargetingRequestData& RequestData = FTargetingRequestData::FindOrAdd(TargetingHandle);
		RequestData.Initialize(CompletionDelegate, CompletionDynamicDelegate, this);

#if ENABLE_DRAW_DEBUG
		if (IsTargetingDebugEnabled())
		{
			AddDebugTrackedImmediateTargetRequests(TargetingHandle);
		}
#endif // ENABLE_DRAW_DEBUG

		if (const FTargetingTaskSet** PtrToTaskSet = FTargetingTaskSet::Find(TargetingHandle))
		{
			if (const FTargetingTaskSet* TaskSet = (*PtrToTaskSet))
			{
				RequestData.bComplete = false;

				for (UTargetingTask* Task : TaskSet->Tasks)
				{
					if (Task)
					{
						Task->Init(TargetingHandle);
						Task->Execute(TargetingHandle);
					}
				}

				RequestData.bComplete = true;
				/* Creates a copy of the RequestData, so that the delegates are free to call ExecuteTargetingRequestWithHandleInternal recursively.
				   After this copy, the RequestData reference is considered as invalid. */
				FTargetingRequestData RequestDataCopy = RequestData;
				RequestDataCopy.BroadcastTargetingRequestDelegate(TargetingHandle);

#if ENABLE_DRAW_DEBUG
#if WITH_EDITORONLY_DATA
				if (const FTargetingDefaultResultsSet* ResultSet = FTargetingDefaultResultsSet::Find(TargetingHandle))
				{
					FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
					DebugData.CachedTargetResults = ResultSet->TargetResults;
				}
#endif // WITH_EDITORONLY_DATA
#endif // ENABLE_DRAW_DEBUG
			}
		}
	}
}

void UTargetingSubsystem::StartAsyncTargetingRequestWithHandleInternal(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate)
{
	// store the request data
	FTargetingRequestData& RequestData = FTargetingRequestData::FindOrAdd(TargetingHandle);
	RequestData.Initialize(CompletionDelegate, CompletionDynamicDelegate, this);

	// initialize the request data for async processing
	FTargetingAsyncTaskData& AsyncTaskData = FTargetingAsyncTaskData::FindOrAdd(TargetingHandle);
	AsyncTaskData.InitializeForAsyncProcessing();

#if ENABLE_DRAW_DEBUG
	if (IsTargetingDebugEnabled())
	{
		AddDebugTrackedAsyncTargetRequests(TargetingHandle);
	}
#endif // ENABLE_DRAW_DEBUG

	AsyncTargetingRequests.Add(TargetingHandle);
}

void UTargetingSubsystem::ProcessTargetingRequestTasks(FTargetingRequestHandle& TargetingHandle, float& TimeLeft)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(TargetingSystem);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks"), STAT_TargetingSystem_TickTasks, STATGROUP_TargetingSystem);

	// @note: this function will need to add total time support for a task to mitigate tasks that stall and never complete.

	if (FTargetingAsyncTaskData* AsyncStateData = FTargetingAsyncTaskData::Find(TargetingHandle))
	{
		if (const FTargetingTaskSet** PtrToTaskSet = FTargetingTaskSet::Find(TargetingHandle))
		{
			TARGETING_LOG(Verbose, TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks - Processing TaskSet [0x%08" UPTRINT_x_FMT "] for Handle [%d]"), (*PtrToTaskSet), TargetingHandle.Handle);
			if (const FTargetingTaskSet* TaskSet = (*PtrToTaskSet))
			{
				bool bRequestComplete = false;

				int32 CurrentTaskIndex = AsyncStateData->CurrentAsyncTaskIndex;
				const int32 NumTasks = TaskSet->Tasks.Num();
				while (CurrentTaskIndex < NumTasks)
				{
					const double StepStartTime = FPlatformTime::Seconds();

					TARGETING_LOG(VeryVerbose, TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks - Processing Async Index [%d] of [%d] for Handle [%d]"), CurrentTaskIndex, NumTasks, TargetingHandle.Handle);

					if (UTargetingTask* Task = TaskSet->Tasks[CurrentTaskIndex])
					{
						TARGETING_LOG(VeryVerbose, TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks - Started Processing Task [%s]"), *(GetNameSafe(Task)));

						if (AsyncStateData->CurrentAsyncTaskState == ETargetingTaskAsyncState::Unitialized)
						{
							Task->Init(TargetingHandle);
						}

						if (AsyncStateData->CurrentAsyncTaskState == ETargetingTaskAsyncState::Initialized)
						{
							Task->Execute(TargetingHandle);
						}

						// @note: if the task is still executing (it has async support internally), we are done
						// until it is done
						if (AsyncStateData->CurrentAsyncTaskState == ETargetingTaskAsyncState::Executing)
						{
							TARGETING_LOG(VeryVerbose, TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks - Waiting on Task [%s] to finish."), *(GetNameSafe(Task)));
							break;
						}

						// if we got here then the task should be complete
						ensure(AsyncStateData->CurrentAsyncTaskState == ETargetingTaskAsyncState::Completed);
						AsyncStateData->CurrentAsyncTaskState = ETargetingTaskAsyncState::Unitialized;

						++CurrentTaskIndex;
						if (CurrentTaskIndex >= NumTasks)
						{
							bRequestComplete = true;
						}
						else
						{
							AsyncStateData->CurrentAsyncTaskIndex = CurrentTaskIndex;

							// if we are out of time, save the task for the next time
							if (TargetingSystemCVars::bUseAsyncTargetingTimeSlicing)
							{
								const double StepDuration = (FPlatformTime::Seconds() - StepStartTime);
								TARGETING_LOG(VeryVerbose, TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks - Finished Processing Task [%s] in [%f] seconds."), *(GetNameSafe(Task)), StepDuration);

								TimeLeft -= StepDuration;
								if (TimeLeft <= 0.0f)
								{
									break;
								}
							}
						}
					}
					else
					{
						TARGETING_LOG(Warning, TEXT("UTargetingSubsystem::ProcessTargetingRequestTasks - Encountered an empty Task, skipping."));
						++CurrentTaskIndex;
						if (CurrentTaskIndex >= NumTasks)
						{
							bRequestComplete = true;
						}
					}
				}

				if (bRequestComplete)
				{
					if (FTargetingRequestData* RequestData = FTargetingRequestData::Find(TargetingHandle))
					{
						RequestData->bComplete = true;
						RequestData->BroadcastTargetingRequestDelegate(TargetingHandle);

#if ENABLE_DRAW_DEBUG
#if WITH_EDITORONLY_DATA
						if (const FTargetingDefaultResultsSet* ResultSet = FTargetingDefaultResultsSet::Find(TargetingHandle))
						{
							FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
							DebugData.CachedTargetResults = ResultSet->TargetResults;
						}
#endif // WITH_EDITORONLY_DATA
#endif // ENABLE_DRAW_DEBUG
					}
				}
			}
		}
	}
}

FTargetingSourceContext UTargetingSubsystem::GetTargetingSourceContext(FTargetingRequestHandle TargetingHandle) const
{
	if (TargetingHandle.IsValid())
	{
		FTargetingSourceContext& SourceContext = FTargetingSourceContext::FindOrAdd(TargetingHandle);
		return SourceContext;
	}

	return FTargetingSourceContext();
}

void UTargetingSubsystem::GetTargetingResultsActors(FTargetingRequestHandle TargetingHandle, TArray<AActor*>& Targets) const
{
	if (TargetingHandle.IsValid())
	{
		if (FTargetingDefaultResultsSet* Results = FTargetingDefaultResultsSet::Find(TargetingHandle))
		{
			for (const FTargetingDefaultResultData& ResultData : Results->TargetResults)
			{
				if (AActor* Target = ResultData.HitResult.GetActor())
				{
					Targets.Add(Target);
				}
			}
		}
	}
}

void UTargetingSubsystem::GetTargetingResults(FTargetingRequestHandle TargetingHandle, TArray<FHitResult>& OutTargets) const
{
	if (TargetingHandle.IsValid())
	{
		if (FTargetingDefaultResultsSet* Results = FTargetingDefaultResultsSet::Find(TargetingHandle))
		{
			for (const FTargetingDefaultResultData& ResultData : Results->TargetResults)
			{
				OutTargets.Add(ResultData.HitResult);
			}
		}
	}
}

void UTargetingSubsystem::OverrideCollisionQueryTaskData(FTargetingRequestHandle TargetingHandle, const FCollisionQueryTaskData& CollisionQueryDataOverride)
{
	if (TargetingHandle.IsValid())
	{
		FCollisionQueryTaskData& AddedCollisionQueryDataOverride = UE::TargetingSystem::TTargetingDataStore<FCollisionQueryTaskData>::FindOrAdd(TargetingHandle);
		AddedCollisionQueryDataOverride = CollisionQueryDataOverride;
	}
}

UTargetingTask* UTargetingSubsystem::FindCurrentExecutingTask(FTargetingRequestHandle Handle) const
{
	if (Handle.IsValid())
	{
		if (FTargetingAsyncTaskData* TaskData = FTargetingAsyncTaskData::Find(Handle))
		{
			const int32 CurrentTaskIndex = TaskData->CurrentAsyncTaskIndex;

			if (TaskData->CurrentAsyncTaskState == ETargetingTaskAsyncState::Executing)
			{
				if (const FTargetingTaskSet** PtrToTaskSet = FTargetingTaskSet::Find(Handle))
				{
					if (const FTargetingTaskSet* TaskSet = (*PtrToTaskSet))
					{
						if (ensure(TaskSet->Tasks.IsValidIndex(CurrentTaskIndex)))
						{
							return TaskSet->Tasks[CurrentTaskIndex];
						}
					}
				}
			}
		}
	}

	return nullptr;
}

void UTargetingSubsystem::HandlePreLoadMap(const FString& MapName)
{
	/* clean up all async requests which will remove all references keeping the old world from being cleaned up */
	ClearAsyncRequests();
}

void UTargetingSubsystem::ClearAsyncRequests()
{
	for (FTargetingRequestHandle AsyncHandle : AsyncTargetingRequests)
	{
		ReleaseTargetRequestHandle(AsyncHandle);
	}

	AsyncTargetingRequests.Empty();
}

#if ENABLE_DRAW_DEBUG

bool UTargetingSubsystem::IsTargetingDebugEnabled()
{
	return TargetingSystemCVars::bEnableTargetingDebugging;
}

float UTargetingSubsystem::GetOverrideTargetingLifeTime()
{
	return TargetingSystemCVars::OverrideTargetingLifeTime;
}

/* static */
void UTargetingSubsystem::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	if (IsTargetingDebugEnabled())
	{
		if (DisplayInfo.IsDisplayOn(TEXT("TargetingSystem")))
		{
			if (UWorld* World = HUD->GetWorld())
			{
				if (UTargetingSubsystem* TargetingSubsystem = UGameInstance::GetSubsystem<UTargetingSubsystem>(World->GetGameInstance()))
				{
					TargetingSubsystem->DisplayDebug(Canvas, DisplayInfo, YL, YPos);
				}
			}
		}
	}
}


void UTargetingSubsystem::DisplayDebug(class UCanvas* Canvas, const class FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	FTargetingDebugInfo DebugInfo;

	DebugInfo.bPrintToLog = TargetingSystemCVars::bPrintTargetingDebugToLog;
	DebugInfo.Canvas = Canvas;
	DebugInfo.XPos = 0.f;
	DebugInfo.YPos = YPos;
	DebugInfo.OriginalX = 0.f;
	DebugInfo.OriginalY = YPos;
	DebugInfo.MaxY = Canvas->ClipY - 150.f; // Give some padding for any non-columnizing debug output following this output
	DebugInfo.NewColumnYPadding = 30.f;

	Debug_Internal(DebugInfo);

	YPos = DebugInfo.YPos;
	YL = DebugInfo.YL;
}

void UTargetingSubsystem::Debug_Internal(struct FTargetingDebugInfo& Info)
{
	FString DebugTitle("Targeting System Debug Info");

	if (Info.Canvas)
	{
		Info.Canvas->SetDrawColor(FColor::White);
		FFontRenderInfo RenderInfo = FFontRenderInfo();
		RenderInfo.bEnableShadow = true;
		Info.Canvas->DrawText(GEngine->GetLargeFont(), DebugTitle, Info.XPos + 4.f, 10.f, 1.5f, 1.5f, RenderInfo);
	}
	else
	{
		DebugLine(Info, DebugTitle, 0.f, 0.f);
	}

	float MaxCharHeight = 10;

	DebugLine(Info, TEXT(""), 0.f, 0.f, 1);
	DebugLine(Info, TEXT("Immediate Targeting Requests:"), 4.0f, 0.0f);

	for (FTargetingRequestHandle& Handle : DebugTrackedImmediateTargetRequests)
	{
		DebugForHandle_Internal(Handle, Info);
	}

	DebugLine(Info, TEXT(""), 0.f, 0.f, 1);
	DebugLine(Info, TEXT("Async Targeting Requests:"), 4.0f, 0.0f);

	for (FTargetingRequestHandle& Handle : DebugTrackedAsyncTargetRequests)
	{
		DebugForHandle_Internal(Handle, Info);
	}

	if (Info.XPos > Info.OriginalX)
	{
		// We flooded to new columns, returned YPos should be max Y (and some padding)
		Info.YPos = Info.MaxY + MaxCharHeight * 2.f;
	}
	Info.YL = MaxCharHeight;
}

void UTargetingSubsystem::DebugForHandle_Internal(FTargetingRequestHandle& Handle, struct FTargetingDebugInfo& Info)
{
#if WITH_EDITORONLY_DATA
	if (Info.Canvas)
	{
		Info.Canvas->SetDrawColor(FColor::White);
	}

	if (Handle.IsValid())
	{
		const FTargetingDebugData* DebugData = FTargetingDebugData::Find(Handle);

		FString HandleString = FString::Printf(TEXT("Targeting Handle [%d]"), Handle.Handle);
		DebugLine(Info, HandleString, 8.0f, 0.0f);

		// dump preset
		FString PresetName = TEXT("Targeting Preset: <none>");
		const UTargetingPreset* RequestPreset = nullptr;
		if (DebugData)
		{
			RequestPreset = DebugData->TargetingPreset;
			PresetName = FString::Printf(TEXT("Targeting Preset: %s"), *GetNameSafe(RequestPreset));
		}
		DebugLine(Info, PresetName, 8.0f, 0.0f);

		// dump tasks
		FString TasksString = TEXT("Tasks:");
		DebugLine(Info, TasksString, 8.0f, 0.0f);
		if (const FTargetingTaskSet** TaskSetPtr = FTargetingTaskSet::Find(Handle))
		{
			if (const FTargetingTaskSet* TaskSet = (*TaskSetPtr))
			{
				for (const UTargetingTask* Task : TaskSet->Tasks)
				{
					if (Task)
					{
						FString TaskString;
						TaskString = FString::Printf(TEXT("%s"), *GetNameSafe(Task));

						if (Info.Canvas)
						{
							Info.Canvas->SetDrawColor(FColor::White);
						}
						DebugLine(Info, TaskString, 12.0f, 0.0f);

						Task->DrawDebug(this, Info, Handle, 16.0f, 0.0f);
					}
				}
			}
		}

		// dump results
		if (Info.Canvas)
		{
			Info.Canvas->SetDrawColor(FColor::White);
		}

		DebugLine(Info, TEXT("Targets:"), 8.0f, 0.0f);
		if (DebugData && DebugData->CachedTargetResults.Num() > 0)
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::Green);
			}

			FString TargetsString;
			for (const FTargetingDefaultResultData& Result : DebugData->CachedTargetResults)
			{
				const AActor* TargetActor = Result.HitResult.GetActor();
				if (TargetActor)
				{
					const float Score = Result.Score;

					TargetsString = FString::Printf(TEXT("%s (%f)"), *GetNameSafe(TargetActor), Score);
					DebugLine(Info, TargetsString, 12.0f, 0.0f);
				}
			}
		}

		if (Info.Canvas)
		{
			Info.Canvas->SetDrawColor(FColor::White);
		}
		DebugLine(Info, TEXT(""), 0.0f, 0.0f, 2);
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSubsystem::AccumulateScreenPos(FTargetingDebugInfo& Info)
{
	const float ColumnWidth = Info.Canvas ? Info.Canvas->ClipX * 0.4f : 0.f;

	float NewY = Info.YPos + Info.YL;
	if (NewY > Info.MaxY)
	{
		// Need new column, reset Y to original height
		NewY = Info.NewColumnYPadding;
		Info.XPos += ColumnWidth;
	}
	Info.YPos = NewY;
}

void UTargetingSubsystem::DebugLine(struct FTargetingDebugInfo& Info, FString Str, float XOffset, float YOffset, int32 MinTextRowsToAdvance /*= 0*/)
{
	if (Info.Canvas)
	{
		FFontRenderInfo RenderInfo = FFontRenderInfo();
		RenderInfo.bEnableShadow = true;
		if (const UFont* Font = GEngine->GetTinyFont())
		{
			float ScaleY = 1.f;
			Info.YL = Info.Canvas->DrawText(Font, Str, Info.XPos + XOffset, Info.YPos, 1.f, ScaleY, RenderInfo);
			if (Info.YL < MinTextRowsToAdvance * (Font->GetMaxCharHeight() * ScaleY))
			{
				Info.YL = MinTextRowsToAdvance * (Font->GetMaxCharHeight() * ScaleY);
			}
			AccumulateScreenPos(Info);
		}
	}

	if (Info.bPrintToLog)
	{
		FString LogStr;
		for (int32 i = 0; i < (int32)XOffset; ++i)
		{
			LogStr += TEXT(" ");
		}
		LogStr += Str;
		TARGETING_LOG(Warning, TEXT("%s"), *LogStr);
	}
}

void UTargetingSubsystem::AddDebugTrackedImmediateTargetRequests(FTargetingRequestHandle TargetingHandle) const
{
	int32 Index = CurrentImmediateRequestIndex++;
	if (Index >= TargetingSystemCVars::TotalDebugRecentRequestsTracked)
	{
		Index = 0;
		CurrentImmediateRequestIndex = 0;
	}

	if (!DebugTrackedImmediateTargetRequests.IsValidIndex(Index))
	{
		DebugTrackedImmediateTargetRequests.SetNumZeroed(TargetingSystemCVars::TotalDebugRecentRequestsTracked);
	}

	FTargetingRequestHandle PreviousHandle = DebugTrackedImmediateTargetRequests[Index];
	ReleaseTargetRequestHandle(PreviousHandle);

	DebugTrackedImmediateTargetRequests[Index] = TargetingHandle;
}

void UTargetingSubsystem::AddDebugTrackedAsyncTargetRequests(FTargetingRequestHandle TargetingHandle) const
{
	int32 Index = CurrentAsyncRequestIndex++;
	if (Index >= TargetingSystemCVars::TotalDebugRecentRequestsTracked)
	{
		Index = 0;
		CurrentAsyncRequestIndex = 0;
	}

	if (!DebugTrackedAsyncTargetRequests.IsValidIndex(Index))
	{
		DebugTrackedAsyncTargetRequests.SetNumZeroed(TargetingSystemCVars::TotalDebugRecentRequestsTracked);
	}

	FTargetingRequestHandle PreviousHandle = DebugTrackedAsyncTargetRequests[Index];
	ReleaseTargetRequestHandle(PreviousHandle);

	DebugTrackedAsyncTargetRequests[Index] = TargetingHandle;
}

#endif // ENABLE_DRAW_DEBUG
