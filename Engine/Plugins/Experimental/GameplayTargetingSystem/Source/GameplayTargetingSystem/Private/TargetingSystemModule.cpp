// Copyright Epic Games, Inc. All Rights Reserved.
#include "TargetingSystemModule.h"

#include "TargetingSystem/TargetingSubsystem.h"
#include "Types/TargetingSystemLogs.h"

#if ENABLE_DRAW_DEBUG
#include "GameFramework/HUD.h"
#endif // ENABLE_DRAW_DEBUG

IMPLEMENT_MODULE(FTargetingSystemModule, TargetingSystem)
DEFINE_LOG_CATEGORY(LogTargetingSystem);


void FTargetingSystemModule::StartupModule()
{
#if ENABLE_DRAW_DEBUG
	if (!IsRunningDedicatedServer())
	{
		AHUD::OnShowDebugInfo.AddStatic(&UTargetingSubsystem::OnShowDebugInfo);
	}
#endif // ENABLE_DRAW_DEBUG
}

void FTargetingSystemModule::ShutdownModule()
{
}
