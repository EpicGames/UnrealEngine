// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/CollisionQueryTaskData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollisionQueryTaskData)

DEFINE_TARGETING_DATA_STORE(FCollisionQueryTaskData)

void FCollisionQueryTaskData::AddStructReferencedObjects(class FReferenceCollector& Collector)
{
	for (TObjectPtr<AActor>& ReferencedActor : IgnoredActors)
	{
		Collector.AddReferencedObject(ReferencedActor);
	}
}
