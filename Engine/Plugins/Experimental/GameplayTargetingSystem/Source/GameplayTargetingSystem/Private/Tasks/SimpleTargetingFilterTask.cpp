// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SimpleTargetingFilterTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleTargetingFilterTask)

bool USimpleTargetingFilterTask::ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const
{
	return BP_ShouldFilterTarget(TargetingHandle, TargetData);
}
