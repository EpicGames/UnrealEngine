// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SimpleTargetingSelectionTask.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleTargetingSelectionTask)

void USimpleTargetingSelectionTask::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
	Super::Execute(TargetingHandle);

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Executing);

	if (FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		// Call the blueprint function
		SelectTargets(TargetingHandle, *SourceContext);
	}
	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

bool USimpleTargetingSelectionTask::AddTargetActor(const FTargetingRequestHandle& TargetingHandle, AActor* Actor) const
{
	if (Actor)
	{
		FTargetingDefaultResultsSet& ResultsSet = FTargetingDefaultResultsSet::FindOrAdd(TargetingHandle);
		if (!ResultsSet.TargetResults.FindByPredicate([Actor](const FTargetingDefaultResultData& ResultData){ return ResultData.HitResult.GetActor() == Actor; }))
		{
			FTargetingDefaultResultData& ResultData = ResultsSet.TargetResults.AddDefaulted_GetRef();
			ResultData.HitResult.HitObjectHandle = FActorInstanceHandle(Actor);
			ResultData.HitResult.Location = Actor->GetActorLocation();
			if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
			{
				if (SourceContext->SourceActor)
				{
					ResultData.HitResult.Distance = FVector::Distance(SourceContext->SourceActor->GetActorLocation(), Actor->GetActorLocation());
				}
			}
			return true;
		}
	}
	return false;
}

bool USimpleTargetingSelectionTask::AddHitResult(const FTargetingRequestHandle& TargetingHandle, const FHitResult& HitResult) const
{
	FTargetingDefaultResultsSet& ResultsSet = FTargetingDefaultResultsSet::FindOrAdd(TargetingHandle);
	if (!ResultsSet.TargetResults.FindByPredicate([HitResult](const FTargetingDefaultResultData& ResultData)
	{
		bool bIsSameActor = ResultData.HitResult.GetActor() == HitResult.GetActor();
		bool bHasSameComponent = ResultData.HitResult.GetComponent() == HitResult.GetComponent();
		return bIsSameActor && bHasSameComponent;
	}))
	{
		FTargetingDefaultResultData& ResultData = ResultsSet.TargetResults.AddDefaulted_GetRef();
		ResultData.HitResult = HitResult;
		if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
		{
			if (SourceContext->SourceActor && HitResult.HasValidHitObjectHandle())
			{
				ResultData.HitResult.Distance = FVector::Distance(SourceContext->SourceActor->GetActorLocation(), HitResult.GetActor()->GetActorLocation());
			}
		}
		return true;
	}
	return false;
}
