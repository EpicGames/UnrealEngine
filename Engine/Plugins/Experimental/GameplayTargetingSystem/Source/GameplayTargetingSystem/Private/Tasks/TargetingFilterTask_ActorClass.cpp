// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tasks/TargetingFilterTask_ActorClass.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingFilterTask_ActorClass)


UTargetingFilterTask_ActorClass::UTargetingFilterTask_ActorClass(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

bool UTargetingFilterTask_ActorClass::ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const
{
	if (AActor* TargetActor = TargetData.HitResult.GetActor())
	{
		// if the target is one of these classes, filter it out
		for (UClass* ClassFilter : IgnoredActorClassFilters)
		{
			if (TargetActor->IsA(ClassFilter))
			{
				return true;
			}
		}

		// if the target is one of these classes, do not filter it out
		for (UClass* ClassFilter : RequiredActorClassFilters)
		{
			if (TargetActor->IsA(ClassFilter))
			{
				return false;
			}
		}

		// if we do not have required class filters, we do NOT want to filter this target
		return (RequiredActorClassFilters.Num() > 0);
	}

	return true;
}
