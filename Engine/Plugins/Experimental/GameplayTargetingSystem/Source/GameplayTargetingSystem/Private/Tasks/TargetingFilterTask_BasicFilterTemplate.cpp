// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tasks/TargetingFilterTask_BasicFilterTemplate.h"

#include "GameFramework/Actor.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Types/TargetingSystemTypes.h"

#if ENABLE_DRAW_DEBUG
#include "Engine/Canvas.h"
#endif // ENABLE_DRAW_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingFilterTask_BasicFilterTemplate)


UTargetingFilterTask_BasicFilterTemplate::UTargetingFilterTask_BasicFilterTemplate(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UTargetingFilterTask_BasicFilterTemplate::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
	Super::Execute(TargetingHandle);

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Executing);

#if ENABLE_DRAW_DEBUG
	ResetFilteredTarget(TargetingHandle);
#endif // ENABLE_DRAW_DEBUG

	if (TargetingHandle.IsValid())
	{
		if (FTargetingDefaultResultsSet* ResultData = FTargetingDefaultResultsSet::Find(TargetingHandle))
		{
			const int32 NumTargets = ResultData->TargetResults.Num();
			for (int32 TargetIterator = NumTargets - 1; TargetIterator >= 0; --TargetIterator)
			{
				const FTargetingDefaultResultData& TargetResult = ResultData->TargetResults[TargetIterator];
				if (ShouldFilterTarget(TargetingHandle, TargetResult))
				{
#if ENABLE_DRAW_DEBUG
					AddFilteredTarget(TargetingHandle, TargetResult);
#endif // ENABLE_DRAW_DEBUG

					ResultData->TargetResults.RemoveAtSwap(TargetIterator, EAllowShrinking::No);
				}
			}
		}
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

bool UTargetingFilterTask_BasicFilterTemplate::ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const
{
	return false;
}

#if ENABLE_DRAW_DEBUG

void UTargetingFilterTask_BasicFilterTemplate::DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
		if (!ScratchPadString.IsEmpty())
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::Yellow);
			}

			FString TaskString = FString::Printf(TEXT("Filtered : %s"), *ScratchPadString);
			TargetingSubsystem->DebugLine(Info, TaskString, XOffset, YOffset, MinTextRowsToAdvance);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingFilterTask_BasicFilterTemplate::AddFilteredTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const
{
#if WITH_EDITORONLY_DATA
	if (AActor* FilteredActor = TargetData.HitResult.GetActor())
	{
		if (FilteredActor && UTargetingSubsystem::IsTargetingDebugEnabled())
		{
			FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
			FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));

			if (ScratchPadString.IsEmpty())
			{
				ScratchPadString = FString::Printf(TEXT("%s"), *GetNameSafe(FilteredActor));
			}
			else
			{
				ScratchPadString += FString::Printf(TEXT(", %s"), *GetNameSafe(FilteredActor));
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingFilterTask_BasicFilterTemplate::ResetFilteredTarget(const FTargetingRequestHandle& TargetingHandle) const
{
#if WITH_EDITORONLY_DATA
	FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
	FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
	ScratchPadString.Reset();
#endif // WITH_EDITORONLY_DATA
}

#endif // ENABLE_DRAW_DEBUG

