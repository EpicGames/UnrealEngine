// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/TargetingFilterTask_SortByDistance.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingFilterTask_SortByDistance)

float UTargetingFilterTask_SortByDistance::GetScoreForTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const
{
	FVector SourceLocation = GetSourceLocation(TargetingHandle);
	if (AActor* TargetActor = TargetData.HitResult.GetActor())
	{
		FVector TargetLocation;
		
		if (bUseDistanceToNearestBlockingCollider)
		{
			const float DistanceToCollision = TargetActor->ActorGetDistanceToCollision(SourceLocation, DistanceToCollisionChannel, TargetLocation);

			if (DistanceToCollision >= 0.f)
			{
				return DistanceToCollision * DistanceToCollision;
			}
		}

		TargetLocation = TargetActor->GetActorLocation();

		return FVector::DistSquared(SourceLocation, TargetLocation);
	}
	return 0.f;
}

FVector UTargetingFilterTask_SortByDistance::GetSourceLocation(const FTargetingRequestHandle& TargetingHandle) const
{
	FVector SourceLocation = FVector::ZeroVector;
	if (FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			SourceLocation = SourceContext->SourceActor->GetActorLocation();
		}
		else if (!SourceContext->SourceLocation.IsZero())
		{
			SourceLocation = SourceContext->SourceLocation;
		}
	}

	return SourceLocation;
}