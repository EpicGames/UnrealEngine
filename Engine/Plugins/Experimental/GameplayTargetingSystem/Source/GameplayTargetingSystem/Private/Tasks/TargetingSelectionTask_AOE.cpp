// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tasks/TargetingSelectionTask_AOE.h"

#include "CollisionQueryParams.h"
#include "Tasks/CollisionQueryTaskData.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/EngineTypes.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Types/TargetingSystemDataStores.h"
#include "Types/TargetingSystemLogs.h"

#if ENABLE_DRAW_DEBUG
#include "Engine/Canvas.h"
#endif // ENABLE_DRAW_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingSelectionTask_AOE)

namespace TargetingSystemCVars
{
	static bool bEnableComplexTracingAOE = true;
	FAutoConsoleVariableRef CVarEnableComplexTracingAOE(
		TEXT("ts.AOE.EnableComplexTracingAOE"),
		bEnableComplexTracingAOE,
		TEXT("When enabled, allows users to do complex traces by setting bShouldTraceComplex to true.")
	);
}

UTargetingSelectionTask_AOE::UTargetingSelectionTask_AOE(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	CollisionChannel = ECC_PhysicsBody;
	bIgnoreSourceActor = false;
	bIgnoreInstigatorActor = false;
	bUseRelativeOffset = false;
}

void UTargetingSelectionTask_AOE::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
	Super::Execute(TargetingHandle);

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Executing);

	// @note: There isn't Async Overlap support based on Primitive Component, so even if using async targeting, it will
	// run this task in "immediate" mode.
	if (IsAsyncTargetingRequest(TargetingHandle) && (ShapeType != ETargetingAOEShape::SourceComponent))
	{
		ExecuteAsyncTrace(TargetingHandle);
	}
	else
	{
		ExecuteImmediateTrace(TargetingHandle);
	}
}

void UTargetingSelectionTask_AOE::ExecuteImmediateTrace(const FTargetingRequestHandle& TargetingHandle) const
{
#if ENABLE_DRAW_DEBUG
	ResetDebugString(TargetingHandle);
#endif // ENABLE_DRAW_DEBUG

	const UWorld* World = GetSourceContextWorld(TargetingHandle);
	if (World && TargetingHandle.IsValid())
	{
		const FVector SourceLocation = GetSourceLocation(TargetingHandle) + GetSourceOffset(TargetingHandle);
		const FQuat SourceRotation = GetSourceRotation(TargetingHandle) * GetSourceRotationOffset(TargetingHandle).Quaternion();

		TArray<FOverlapResult> OverlapResults;
		if (ShapeType == ETargetingAOEShape::SourceComponent)
		{
			if (const UPrimitiveComponent* CollisionComponent = GetCollisionComponent(TargetingHandle))
			{
				FComponentQueryParams ComponentQueryParams(SCENE_QUERY_STAT(UTargetingSelectionTask_AOE_Component));
				InitCollisionParams(TargetingHandle, ComponentQueryParams);
				World->ComponentOverlapMulti(OverlapResults, CollisionComponent, CollisionComponent->GetComponentLocation(), CollisionComponent->GetComponentRotation(), ComponentQueryParams);
			}
			else
			{
				TARGETING_LOG(Warning, TEXT("UTargetingSelectionTask_AOE::Execute - Failed to find a collision component w/ tag [%s] for a SourceComponent ShapeType."), *ComponentTag.ToString());
			}
		}
		else
		{
			const FCollisionShape CollisionShape = GetCollisionShape();
			FCollisionQueryParams OverlapParams(TEXT("UTargetingSelectionTask_AOE"), SCENE_QUERY_STAT_ONLY(UTargetingSelectionTask_AOE_Shape), false);
			InitCollisionParams(TargetingHandle, OverlapParams);

			if (CollisionObjectTypes.Num() > 0)
			{
				FCollisionObjectQueryParams ObjectParams;
				for (auto Iter = CollisionObjectTypes.CreateConstIterator(); Iter; ++Iter)
				{
					const ECollisionChannel& Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
					ObjectParams.AddObjectTypesToQuery(Channel);
				}

				World->OverlapMultiByObjectType(OverlapResults, SourceLocation, SourceRotation, ObjectParams, CollisionShape, OverlapParams);
			}
			else if (CollisionProfileName.Name != TEXT("NoCollision"))
			{
				World->OverlapMultiByProfile(OverlapResults, SourceLocation, SourceRotation, CollisionProfileName.Name, CollisionShape, OverlapParams);
			}
			else
			{
				World->OverlapMultiByChannel(OverlapResults, SourceLocation, SourceRotation, CollisionChannel, CollisionShape, OverlapParams);
			}

#if ENABLE_DRAW_DEBUG
			if (UTargetingSubsystem::IsTargetingDebugEnabled())
			{
				DebugDrawBoundingVolume(TargetingHandle, FColor::Red);
			}
#endif // ENABLE_DRAW_DEBUG
		}

		ProcessOverlapResults(TargetingHandle, OverlapResults);
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

void UTargetingSelectionTask_AOE::DebugDrawBoundingVolume(const FTargetingRequestHandle& TargetingHandle, const FColor& Color, const FOverlapDatum* OverlapDatum) const
{
#if ENABLE_DRAW_DEBUG
	const UWorld* World = GetSourceContextWorld(TargetingHandle);
	const FVector SourceLocation = OverlapDatum ? OverlapDatum->Pos : GetSourceLocation(TargetingHandle) + GetSourceOffset(TargetingHandle);
	const FQuat SourceRotation = OverlapDatum ? OverlapDatum->Rot : GetSourceRotation(TargetingHandle) * GetSourceRotationOffset(TargetingHandle).Quaternion();
	const FCollisionShape CollisionShape = GetCollisionShape();

	const bool bPersistentLines = false;
	const float LifeTime = UTargetingSubsystem::GetOverrideTargetingLifeTime();
	const uint8 DepthPriority = 0;
	const float Thickness = 2.0f;

	switch (ShapeType)
	{
	case ETargetingAOEShape::Box:
		DrawDebugBox(World, SourceLocation, CollisionShape.GetExtent(), SourceRotation,
			Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
		break;
	case ETargetingAOEShape::Sphere:
		DrawDebugCapsule(World, SourceLocation, CollisionShape.GetSphereRadius(), CollisionShape.GetSphereRadius(), SourceRotation,
			Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
		break;
	case ETargetingAOEShape::Capsule:
		DrawDebugCapsule(World, SourceLocation, CollisionShape.GetCapsuleHalfHeight(), CollisionShape.GetCapsuleRadius(), SourceRotation,
			Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
		break;
	case ETargetingAOEShape::Cylinder:
		{
			const FVector RotatedExtent = SourceRotation * FVector(0, 0, CollisionShape.GetExtent().Z);
			DrawDebugCylinder(World, SourceLocation - RotatedExtent, SourceLocation + RotatedExtent, CollisionShape.GetExtent().X, 32,
				Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			break;
		}
	}
#endif // ENABLE_DRAW_DEBUG
}

void UTargetingSelectionTask_AOE::SetShapeType(ETargetingAOEShape InShapeType)
{
	ShapeType = InShapeType;
}

void UTargetingSelectionTask_AOE::SetHalfExtent(FVector InHalfExtent)
{
	HalfExtent = InHalfExtent;
}

void UTargetingSelectionTask_AOE::SetRadius(FScalableFloat InRadius)
{
	Radius = InRadius;
}

void UTargetingSelectionTask_AOE::SetHalfHeight(FScalableFloat InHalfHeight)
{
	HalfHeight = InHalfHeight;
}

void UTargetingSelectionTask_AOE::ExecuteAsyncTrace(const FTargetingRequestHandle& TargetingHandle) const
{
	UWorld* World = GetSourceContextWorld(TargetingHandle);
	if (World && TargetingHandle.IsValid())
	{
		const FVector SourceLocation = GetSourceLocation(TargetingHandle) + GetSourceOffset(TargetingHandle);
		const FQuat SourceRotation = GetSourceRotation(TargetingHandle) * GetSourceRotationOffset(TargetingHandle).Quaternion();

		FCollisionShape CollisionShape = GetCollisionShape();
		FCollisionQueryParams OverlapParams(TEXT("UTargetingSelectionTask_AOE"), SCENE_QUERY_STAT_ONLY(UTargetingSelectionTask_AOE_Shape), false);
		InitCollisionParams(TargetingHandle, OverlapParams);

		FOverlapDelegate Delegate = FOverlapDelegate::CreateUObject(this, &UTargetingSelectionTask_AOE::HandleAsyncOverlapComplete, TargetingHandle);
		if (CollisionObjectTypes.Num() > 0)
		{
			FCollisionObjectQueryParams ObjectParams;
			for (auto Iter = CollisionObjectTypes.CreateConstIterator(); Iter; ++Iter)
			{
				const ECollisionChannel& Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
				ObjectParams.AddObjectTypesToQuery(Channel);
			}

			World->AsyncOverlapByObjectType(SourceLocation, SourceRotation, ObjectParams, CollisionShape, OverlapParams, &Delegate);
		}
		else if (CollisionProfileName.Name != TEXT("NoCollision"))
		{
			World->AsyncOverlapByProfile(SourceLocation, SourceRotation, CollisionProfileName.Name, CollisionShape, OverlapParams, &Delegate);
		}
		else
		{
			World->AsyncOverlapByChannel(SourceLocation, SourceRotation, CollisionChannel, CollisionShape, OverlapParams, FCollisionResponseParams::DefaultResponseParam, &Delegate);
		}
	}
	else
	{
		SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
	}
}

void UTargetingSelectionTask_AOE::HandleAsyncOverlapComplete(const FTraceHandle& InTraceHandle, FOverlapDatum& InOverlapDatum, FTargetingRequestHandle TargetingHandle) const
{
	if (TargetingHandle.IsValid())
	{
#if ENABLE_DRAW_DEBUG
		ResetDebugString(TargetingHandle);

		if (UTargetingSubsystem::IsTargetingDebugEnabled())
		{
			DebugDrawBoundingVolume(TargetingHandle, FColor::Red, &InOverlapDatum);
		}
#endif // ENABLE_DRAW_DEBUG

		ProcessOverlapResults(TargetingHandle, InOverlapDatum.OutOverlaps);
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

void UTargetingSelectionTask_AOE::ProcessOverlapResults(const FTargetingRequestHandle& TargetingHandle, const TArray<FOverlapResult>& Overlaps) const
{
	// process the overlaps
	if (Overlaps.Num() > 0)
	{
		FTargetingDefaultResultsSet& TargetingResults = FTargetingDefaultResultsSet::FindOrAdd(TargetingHandle);
		const FVector SourceLocation = GetSourceLocation(TargetingHandle) + GetSourceOffset(TargetingHandle);
		for (const FOverlapResult& OverlapResult : Overlaps)
		{

			if (!OverlapResult.GetActor())
			{
				continue;
			}

			// cylinders use box overlaps, so a radius check is necessary to constrain it to the bounds of a cylinder
			if (ShapeType == ETargetingAOEShape::Cylinder)
			{
				const float RadiusSquared = (HalfExtent.X * HalfExtent.X);
				const float DistanceSquared = FVector::DistSquared2D(OverlapResult.GetActor()->GetActorLocation(), SourceLocation);
				if (DistanceSquared > RadiusSquared)
				{
					continue;
				}
			}

			bool bAddResult = true;
			for (const FTargetingDefaultResultData& ResultData : TargetingResults.TargetResults)
			{
				if (ResultData.HitResult.GetActor() == OverlapResult.GetActor())
				{
					bAddResult = false;
					break;
				}
			}

			if (bAddResult)
			{
				FTargetingDefaultResultData* ResultData = new(TargetingResults.TargetResults) FTargetingDefaultResultData();
				ResultData->HitResult.HitObjectHandle = OverlapResult.OverlapObjectHandle;
				ResultData->HitResult.Component = OverlapResult.GetComponent();
				ResultData->HitResult.ImpactPoint = OverlapResult.GetActor()->GetActorLocation();
				ResultData->HitResult.Location = OverlapResult.GetActor()->GetActorLocation();
				ResultData->HitResult.bBlockingHit = OverlapResult.bBlockingHit;
				ResultData->HitResult.TraceStart = SourceLocation;
				ResultData->HitResult.Item = OverlapResult.GetItemIndex();
				ResultData->HitResult.Distance = FVector::Distance(OverlapResult.GetActor()->GetActorLocation(), SourceLocation);
			}
		}

#if ENABLE_DRAW_DEBUG
		BuildDebugString(TargetingHandle, TargetingResults.TargetResults);
#endif // ENABLE_DRAW_DEBUG
	}
}

FVector UTargetingSelectionTask_AOE::GetSourceLocation_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			return SourceContext->SourceActor->GetActorLocation();
		}
		
		return SourceContext->SourceLocation;
	}

	return FVector::ZeroVector;
}

FVector UTargetingSelectionTask_AOE::GetSourceOffset_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	// If we want to apply a relative offset, we offset the different base vectors of the local space of the source actor with the values provided
	if(bUseRelativeOffset)
	{
		if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
		{
			if (const AActor* SourceActor = SourceContext->SourceActor)
			{
				return	SourceActor->GetActorForwardVector() * DefaultSourceOffset.X +
						SourceActor->GetActorRightVector() * DefaultSourceOffset.Y +
						SourceActor->GetActorUpVector() * DefaultSourceOffset.Z;
			}
		}
	}

	return DefaultSourceOffset;
}

FQuat UTargetingSelectionTask_AOE::GetSourceRotation_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			return SourceContext->SourceActor->GetActorQuat();
		}
	}

	return FQuat::Identity;
}

FRotator UTargetingSelectionTask_AOE::GetSourceRotationOffset_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return DefaultSourceRotationOffset;
}

FCollisionShape UTargetingSelectionTask_AOE::GetCollisionShape() const
{
	switch (ShapeType)
	{
	case ETargetingAOEShape::Box:
		return FCollisionShape::MakeBox(HalfExtent);

	case ETargetingAOEShape::Cylinder:
		return FCollisionShape::MakeBox(HalfExtent);

	case ETargetingAOEShape::Sphere:
		return FCollisionShape::MakeSphere(Radius.GetValue());

	case ETargetingAOEShape::Capsule:
		return FCollisionShape::MakeCapsule(Radius.GetValue(), HalfHeight.GetValue());
	}

	return FCollisionShape();
}

const UPrimitiveComponent* UTargetingSelectionTask_AOE::GetCollisionComponent(const FTargetingRequestHandle& TargetingHandle) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			TArray<UPrimitiveComponent*> PrimitiveComponents;
			SourceContext->SourceActor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);

			for (UPrimitiveComponent* Component : PrimitiveComponents)
			{
				if (Component && Component->ComponentHasTag(ComponentTag))
				{
					return Component;
				}
			}
		}
	}

	return nullptr;
}

void UTargetingSelectionTask_AOE::InitCollisionParams(const FTargetingRequestHandle& TargetingHandle, FCollisionQueryParams& OutParams) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (bIgnoreSourceActor && SourceContext->SourceActor)
		{
			OutParams.AddIgnoredActor(SourceContext->SourceActor);
		}

		if (bIgnoreInstigatorActor && SourceContext->InstigatorActor)
		{
			OutParams.AddIgnoredActor(SourceContext->InstigatorActor);
		}
	}

	// If we have a data store to override the collision query params from, add that to our collision params
	if (FCollisionQueryTaskData* FoundOverride = UE::TargetingSystem::TTargetingDataStore<FCollisionQueryTaskData>::Find(TargetingHandle))
	{
		OutParams.AddIgnoredActors(FoundOverride->IgnoredActors);
	}

	OutParams.bTraceComplex = TargetingSystemCVars::bEnableComplexTracingAOE && bTraceComplex;
}

#if WITH_EDITOR
bool UTargetingSelectionTask_AOE::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);

	if (bCanEdit && InProperty)
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, CollisionChannel))
		{
			return (CollisionProfileName.Name == TEXT("NoCollision") && CollisionObjectTypes.Num() <= 0);
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, CollisionProfileName))
		{
			return (CollisionObjectTypes.Num() <= 0);
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, HalfExtent))
		{
			return (ShapeType == ETargetingAOEShape::Box || ShapeType == ETargetingAOEShape::Cylinder);
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, Radius))
		{
			return (ShapeType == ETargetingAOEShape::Sphere || ShapeType == ETargetingAOEShape::Capsule);
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, HalfHeight))
		{
			return (ShapeType == ETargetingAOEShape::Capsule);
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, ComponentTag))
		{
			return (ShapeType == ETargetingAOEShape::SourceComponent);
		}
	}

	return true;
}

void UTargetingSelectionTask_AOE::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, HalfExtent) ||
			PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_AOE, ShapeType))
		{
			// If using Cylinder make sure the X and Y match so that we don't get incorrect Overlaps when Y < X
			if (ShapeType == ETargetingAOEShape::Cylinder)
			{
				HalfExtent.Y = HalfExtent.X;
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

#if ENABLE_DRAW_DEBUG

void UTargetingSelectionTask_AOE::DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
		if (!ScratchPadString.IsEmpty())
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::Yellow);
			}

			FString TaskString = FString::Printf(TEXT("Results : %s"), *ScratchPadString);
			TargetingSubsystem->DebugLine(Info, TaskString, XOffset, YOffset, MinTextRowsToAdvance);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSelectionTask_AOE::BuildDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));

		for (const FTargetingDefaultResultData& TargetData : TargetResults)
		{
			if (const AActor* Target = TargetData.HitResult.GetActor())
			{
				if (ScratchPadString.IsEmpty())
				{
					ScratchPadString = FString::Printf(TEXT("%s"), *GetNameSafe(Target));
				}
				else
				{
					ScratchPadString += FString::Printf(TEXT(", %s"), *GetNameSafe(Target));
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSelectionTask_AOE::ResetDebugString(const FTargetingRequestHandle& TargetingHandle) const
{
#if WITH_EDITORONLY_DATA
	FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
	FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
	ScratchPadString.Reset();
#endif // WITH_EDITORONLY_DATA
}

#endif // ENABLE_DRAW_DEBUG

