// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tasks/TargetingSelectionTask_SourceActor.h"

#include "GameFramework/Actor.h"
#include "TargetingSystem/TargetingSubsystem.h"

#if ENABLE_DRAW_DEBUG
#include "Engine/Canvas.h"
#endif // ENABLE_DRAW_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingSelectionTask_SourceActor)


UTargetingSelectionTask_SourceActor::UTargetingSelectionTask_SourceActor(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UTargetingSelectionTask_SourceActor::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
	Super::Execute(TargetingHandle);

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Executing);

#if ENABLE_DRAW_DEBUG
	ResetSourceActorDebugString(TargetingHandle);
#endif // ENABLE_DRAW_DEBUG

	if (TargetingHandle.IsValid())
	{
		if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
		{
			if (SourceContext->SourceActor)
			{
				bool AddSourceActor = true;

				FTargetingDefaultResultsSet& Results = FTargetingDefaultResultsSet::FindOrAdd(TargetingHandle);
				for (const FTargetingDefaultResultData& ResultData : Results.TargetResults)
				{
					if (ResultData.HitResult.GetActor() == SourceContext->SourceActor)
					{
						AddSourceActor = false;
						break;
					}
				}

				if (AddSourceActor)
				{
					FTargetingDefaultResultData* ResultData = new(Results.TargetResults) FTargetingDefaultResultData();
					ResultData->HitResult.HitObjectHandle = FActorInstanceHandle(SourceContext->SourceActor);
					ResultData->HitResult.Location = SourceContext->SourceActor->GetActorLocation();
					ResultData->HitResult.Distance = 0.0f;
					ResultData->HitResult.Time = 0.f;
				}

#if ENABLE_DRAW_DEBUG
				AddSourceActorDebugString(TargetingHandle, SourceContext->SourceActor);
#endif // ENABLE_DRAW_DEBUG
			}
		}
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

#if ENABLE_DRAW_DEBUG

void UTargetingSelectionTask_SourceActor::DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
		if (!ScratchPadString.IsEmpty())
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::Yellow);
			}

			FString TaskString = FString::Printf(TEXT(": %s"), *ScratchPadString);
			TargetingSubsystem->DebugLine(Info, TaskString, XOffset, YOffset, MinTextRowsToAdvance);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSelectionTask_SourceActor::AddSourceActorDebugString(const FTargetingRequestHandle& TargetingHandle, const AActor* SourceActor) const
{
#if WITH_EDITORONLY_DATA
	if (SourceActor && UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));

		ScratchPadString = FString::Printf(TEXT("%s"), *GetNameSafe(SourceActor));
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSelectionTask_SourceActor::ResetSourceActorDebugString(const FTargetingRequestHandle& TargetingHandle) const
{
#if WITH_EDITORONLY_DATA
	FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
	FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
	ScratchPadString.Reset();
#endif // WITH_EDITORONLY_DATA
}

#endif // ENABLE_DRAW_DEBUG
