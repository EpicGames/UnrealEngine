// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tasks/TargetingSelectionTask_Trace.h"

#include "CollisionQueryParams.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "KismetTraceUtils.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Types/TargetingSystemLogs.h"

#if ENABLE_DRAW_DEBUG
#include "Engine/Canvas.h"
#endif // ENABLE_DRAW_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingSelectionTask_Trace)


UTargetingSelectionTask_Trace::UTargetingSelectionTask_Trace(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bMultiTrace = true;
	bComplexTrace = false;
	bIgnoreSourceActor = false;
	bIgnoreInstigatorActor = false;
}

void UTargetingSelectionTask_Trace::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
	Super::Execute(TargetingHandle);

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Executing);

	if (IsAsyncTargetingRequest(TargetingHandle))
	{
		ExecuteAsyncTrace(TargetingHandle);
	}
	else
	{
		ExecuteImmediateTrace(TargetingHandle);
	}
}

FVector UTargetingSelectionTask_Trace::GetSourceLocation_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			return SourceContext->SourceActor->GetActorLocation();
		}
		
		return SourceContext->SourceLocation;
	}

	return FVector::ZeroVector;
}

FVector UTargetingSelectionTask_Trace::GetSourceOffset_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return DefaultSourceOffset;
}

FVector UTargetingSelectionTask_Trace::GetTraceDirection_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	if (ExplicitTraceDirection != FVector::ZeroVector)
	{
		return ExplicitTraceDirection;
	}
	
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			if (APawn* Pawn = Cast<APawn>(SourceContext->SourceActor))
			{
				return Pawn->GetControlRotation().Vector();
			}
			else
			{
				return SourceContext->SourceActor->GetActorForwardVector();
			}
		}
	}

	return FVector::ZeroVector;
}

float UTargetingSelectionTask_Trace::GetTraceLength_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return DefaultTraceLength.GetValue();
}

float UTargetingSelectionTask_Trace::GetSweptTraceRadius_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return DefaultSweptTraceRadius.GetValue();
}

float UTargetingSelectionTask_Trace::GetSweptTraceCapsuleHalfHeight_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return DefaultSweptTraceCapsuleHalfHeight.GetValue();
}

FVector UTargetingSelectionTask_Trace::GetSweptTraceBoxHalfExtents_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return FVector(DefaultSweptTraceBoxHalfExtentX.GetValue(), DefaultSweptTraceBoxHalfExtentY.GetValue(), DefaultSweptTraceBoxHalfExtentZ.GetValue());
}

FRotator UTargetingSelectionTask_Trace::GetSweptTraceRotation_Implementation(const FTargetingRequestHandle& TargetingHandle) const
{
	return DefaultSweptTraceRotation;
}

void UTargetingSelectionTask_Trace::GetAdditionalActorsToIgnore_Implementation(const FTargetingRequestHandle& TargetingHandle, TArray<AActor*>& OutAdditionalActorsToIgnore) const
{
}

FQuat UTargetingSelectionTask_Trace::GetSweptTraceQuat(const FVector& TraceDirection, const FTargetingRequestHandle& TargetingHandle) const
{
	const FQuat AbsoluteQuat = TraceDirection.ToOrientationQuat();
	const FQuat RelativeQuat = GetSweptTraceRotation(TargetingHandle).Quaternion();
	return AbsoluteQuat * RelativeQuat;
}

void UTargetingSelectionTask_Trace::ExecuteImmediateTrace(const FTargetingRequestHandle& TargetingHandle) const
{
	if (UWorld* World = GetSourceContextWorld(TargetingHandle))
	{
#if ENABLE_DRAW_DEBUG
		ResetTraceResultsDebugString(TargetingHandle);
#endif // ENABLE_DRAW_DEBUG

		const FVector Direction = GetTraceDirection(TargetingHandle).GetSafeNormal();
		const FVector Start = (GetSourceLocation(TargetingHandle) + GetSourceOffset(TargetingHandle));
		const FVector End = Start + (Direction * GetTraceLength(TargetingHandle));

		// Only bother calculating the orientation for shapes where orientation matters (i.e not points and not sphere)
		FQuat OrientationQuat = FQuat::Identity;
		if (TraceType != ETargetingTraceType::Line && TraceType != ETargetingTraceType::Sphere)
		{
			OrientationQuat = GetSweptTraceQuat(Direction, TargetingHandle);
		}

		FCollisionQueryParams Params(SCENE_QUERY_STAT(ExecuteImmediateTrace), bComplexTrace);
		InitCollisionParams(TargetingHandle, Params);

		FCollisionShape CollisionShape;
		switch (TraceType)
		{
		case ETargetingTraceType::Sphere:
			CollisionShape = FCollisionShape::MakeSphere(GetSweptTraceRadius(TargetingHandle));
			break;
		case ETargetingTraceType::Capsule:
		{
			const FVector CapsuleShapeVector = FVector(0.0f, GetSweptTraceRadius(TargetingHandle), GetSweptTraceCapsuleHalfHeight(TargetingHandle));
			CollisionShape = FCollisionShape::MakeCapsule(CapsuleShapeVector);
		}
		break;
		case ETargetingTraceType::Box:
			CollisionShape = FCollisionShape::MakeBox(GetSweptTraceBoxHalfExtents(TargetingHandle));
			break;
		}

		bool bHasBlockingHit = false;
		TArray<FHitResult> Hits;

		if (!bMultiTrace)
		{
			// Populate an element in the array for single trace to use
			Hits.SetNum(1);
		}

		if (CollisionProfileName.Name != TEXT("NoCollision"))
		{
			if (TraceType == ETargetingTraceType::Line)
			{
				if (bMultiTrace)
				{
					bHasBlockingHit = World->LineTraceMultiByProfile(Hits, Start, End, CollisionProfileName.Name, Params);
				}
				else
				{
					bHasBlockingHit = World->LineTraceSingleByProfile(Hits[0], Start, End, CollisionProfileName.Name, Params);
				}
			}
			else
			{
				if (bMultiTrace)
				{
					bHasBlockingHit = World->SweepMultiByProfile(Hits, Start, End, OrientationQuat, CollisionProfileName.Name, CollisionShape, Params);
				}
				else
				{
					bHasBlockingHit = World->SweepSingleByProfile(Hits[0], Start, End, OrientationQuat, CollisionProfileName.Name, CollisionShape, Params);
				}
			}
		}
		else
		{
			const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);
			if (TraceType == ETargetingTraceType::Line)
			{
				if (bMultiTrace)
				{
					bHasBlockingHit = World->LineTraceMultiByChannel(Hits, Start, End, CollisionChannel, Params);
				}
				else
				{
					bHasBlockingHit = World->LineTraceSingleByChannel(Hits[0], Start, End, CollisionChannel, Params);
				}
			}
			else
			{
				if (bMultiTrace)
				{
					bHasBlockingHit = World->SweepMultiByChannel(Hits, Start, End, OrientationQuat, CollisionChannel, CollisionShape, Params);
				}
				else
				{
					bHasBlockingHit = World->SweepSingleByChannel(Hits[0], Start, End, OrientationQuat, CollisionChannel, CollisionShape, Params);
				}
			}
		}

#if ENABLE_DRAW_DEBUG
		DrawDebugTrace(TargetingHandle, Start, End, OrientationQuat, bHasBlockingHit, Hits);
#endif // ENABLE_DRAW_DEBUG

		ProcessHitResults(TargetingHandle, Hits);
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

void UTargetingSelectionTask_Trace::ExecuteAsyncTrace(const FTargetingRequestHandle& TargetingHandle) const
{
	if (UWorld* World = GetSourceContextWorld(TargetingHandle))
	{
		AActor* SourceActor = nullptr;
		if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
		{
			SourceActor = SourceContext->SourceActor;
		}
		const FVector Direction = GetTraceDirection(TargetingHandle).GetSafeNormal();
		const FVector Start = (GetSourceLocation(TargetingHandle) + GetSourceOffset(TargetingHandle));
		const FVector End = Start + (Direction * GetTraceLength(TargetingHandle));

		// Only bother calculating the orientation for shapes where orientation matters (i.e not points and not sphere)
		FQuat OrientationQuat = FQuat::Identity;
		if (TraceType != ETargetingTraceType::Line && TraceType != ETargetingTraceType::Sphere)
		{
			OrientationQuat = GetSweptTraceQuat(Direction, TargetingHandle);
		}

		FCollisionQueryParams Params(SCENE_QUERY_STAT(ExecuteAsyncTrace), bComplexTrace);
		InitCollisionParams(TargetingHandle, Params);

		FTraceDelegate Delegate = FTraceDelegate::CreateUObject(this, &UTargetingSelectionTask_Trace::HandleAsyncTraceComplete, TargetingHandle);
		if (CollisionProfileName.Name != TEXT("NoCollision"))
		{
			switch (TraceType)
			{
			case ETargetingTraceType::Sphere:
				World->AsyncSweepByProfile(EAsyncTraceType::Multi, Start, End, FQuat::Identity, CollisionProfileName.Name, FCollisionShape::MakeSphere(GetSweptTraceRadius(TargetingHandle)), Params, &Delegate);
				break;
			case ETargetingTraceType::Capsule:
			{
				const FVector CapsuleShapeVector = FVector(0.0f, GetSweptTraceRadius(TargetingHandle), GetSweptTraceCapsuleHalfHeight(TargetingHandle));
				World->AsyncSweepByProfile(EAsyncTraceType::Multi, Start, End, OrientationQuat, CollisionProfileName.Name, FCollisionShape::MakeCapsule(CapsuleShapeVector), Params, &Delegate);
			}
			break;
			case ETargetingTraceType::Box:
				World->AsyncSweepByProfile(EAsyncTraceType::Multi, Start, End, OrientationQuat, CollisionProfileName.Name, FCollisionShape::MakeBox(GetSweptTraceBoxHalfExtents(TargetingHandle)), Params, &Delegate);
				break;
			default:
			case ETargetingTraceType::Line:
				World->AsyncLineTraceByProfile(EAsyncTraceType::Multi, Start, End, CollisionProfileName.Name, Params, &Delegate);
				break;
			}
		}
		else
		{
			const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);
			switch (TraceType)
			{
			case ETargetingTraceType::Sphere:
				World->AsyncSweepByChannel(EAsyncTraceType::Multi, Start, End, FQuat::Identity, CollisionChannel, FCollisionShape::MakeSphere(GetSweptTraceRadius(TargetingHandle)), Params, FCollisionResponseParams::DefaultResponseParam, &Delegate);
				break;
			case ETargetingTraceType::Capsule:
			{
				const FVector CapsuleShapeVector = FVector(0.0f, GetSweptTraceRadius(TargetingHandle), GetSweptTraceCapsuleHalfHeight(TargetingHandle));
				World->AsyncSweepByChannel(EAsyncTraceType::Multi, Start, End, OrientationQuat, CollisionChannel, FCollisionShape::MakeCapsule(CapsuleShapeVector), Params, FCollisionResponseParams::DefaultResponseParam, &Delegate);
			}
				break;
			case ETargetingTraceType::Box:
				World->AsyncSweepByChannel(EAsyncTraceType::Multi, Start, End, OrientationQuat, CollisionChannel, FCollisionShape::MakeBox(GetSweptTraceBoxHalfExtents(TargetingHandle)), Params, FCollisionResponseParams::DefaultResponseParam, &Delegate);
				break;
			default:
			case ETargetingTraceType::Line:
				World->AsyncLineTraceByChannel(EAsyncTraceType::Multi, Start, End, CollisionChannel, Params, FCollisionResponseParams::DefaultResponseParam, &Delegate);
				break;
			}
		}
	}
	else
	{
		SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
	}
}

void UTargetingSelectionTask_Trace::HandleAsyncTraceComplete(const FTraceHandle& InTraceHandle, FTraceDatum& InTraceDatum, FTargetingRequestHandle TargetingHandle) const
{
	if (TargetingHandle.IsValid())
	{
#if ENABLE_DRAW_DEBUG
		ResetTraceResultsDebugString(TargetingHandle);

		// We have to manually find if there is a blocking hit.
		bool bHasBlockingHit = false;
		for (const FHitResult& HitResult : InTraceDatum.OutHits)
		{
			if (HitResult.bBlockingHit)
			{
				bHasBlockingHit = true;
				break;
			}
		}

		DrawDebugTrace(TargetingHandle, InTraceDatum.Start, InTraceDatum.End, InTraceDatum.Rot, bHasBlockingHit, InTraceDatum.OutHits);

#endif // ENABLE_DRAW_DEBUG

		ProcessHitResults(TargetingHandle, InTraceDatum.OutHits);
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}

void UTargetingSelectionTask_Trace::ProcessHitResults(const FTargetingRequestHandle& TargetingHandle, const TArray<FHitResult>& Hits) const
{
	if (TargetingHandle.IsValid() && Hits.Num() > 0)
	{
		FTargetingDefaultResultsSet& TargetingResults = FTargetingDefaultResultsSet::FindOrAdd(TargetingHandle);
		for (const FHitResult& HitResult : Hits)
		{
			if (!HitResult.GetActor())
			{
				continue;
			}

			bool bAddResult = true;
			for (const FTargetingDefaultResultData& ResultData : TargetingResults.TargetResults)
			{
				if (ResultData.HitResult.GetActor() == HitResult.GetActor())
				{
					bAddResult = false;
					break;
				}
			}

			if (bAddResult)
			{
				FTargetingDefaultResultData* ResultData = new(TargetingResults.TargetResults) FTargetingDefaultResultData();
				ResultData->HitResult = HitResult;
			}
		}

#if ENABLE_DRAW_DEBUG
		BuildTraceResultsDebugString(TargetingHandle, TargetingResults.TargetResults);
#endif // ENABLE_DRAW_DEBUG
	}
}

void UTargetingSelectionTask_Trace::InitCollisionParams(const FTargetingRequestHandle& TargetingHandle, FCollisionQueryParams& OutParams) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (bIgnoreSourceActor && SourceContext->SourceActor)
		{
			OutParams.AddIgnoredActor(SourceContext->SourceActor);
		}

		if (bIgnoreInstigatorActor && SourceContext->InstigatorActor)
		{
			OutParams.AddIgnoredActor(SourceContext->InstigatorActor);
		}

		TArray<AActor*> AdditionalActorsToIgnoreArray;
		GetAdditionalActorsToIgnore(TargetingHandle, AdditionalActorsToIgnoreArray);

		if (AdditionalActorsToIgnoreArray.Num() > 0)
		{
			OutParams.AddIgnoredActors(AdditionalActorsToIgnoreArray);
		}
	}
}

#if WITH_EDITOR
bool UTargetingSelectionTask_Trace::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);

	if (bCanEdit && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, DefaultSweptTraceRadius))
		{
			return (TraceType == ETargetingTraceType::Sphere || TraceType == ETargetingTraceType::Capsule);
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, DefaultSweptTraceCapsuleHalfHeight))
		{
			return (TraceType == ETargetingTraceType::Capsule);
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, DefaultSweptTraceBoxHalfExtentX)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, DefaultSweptTraceBoxHalfExtentY)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, DefaultSweptTraceBoxHalfExtentZ))
		{
			return (TraceType == ETargetingTraceType::Box);
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, DefaultSweptTraceRotation))
		{
			return (TraceType == ETargetingTraceType::Capsule || TraceType == ETargetingTraceType::Box);
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTargetingSelectionTask_Trace, TraceChannel))
		{
			return (CollisionProfileName.Name == TEXT("NoCollision"));
		}
	}

	return true;
}
#endif // WITH_EDITOR

#if ENABLE_DRAW_DEBUG
void UTargetingSelectionTask_Trace::DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
		if (!ScratchPadString.IsEmpty())
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::Yellow);
			}

			FString TaskString = FString::Printf(TEXT("Results : %s"), *ScratchPadString);
			TargetingSubsystem->DebugLine(Info, TaskString, XOffset, YOffset, MinTextRowsToAdvance);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSelectionTask_Trace::DrawDebugTrace(const FTargetingRequestHandle TargetingHandle, const FVector& StartLocation, const FVector& EndLocation, const FQuat& OrientationQuat, const bool bHit, const TArray<FHitResult>& Hits) const
{
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		if (UWorld* World = GetSourceContextWorld(TargetingHandle))
		{
			const float DrawTime = UTargetingSubsystem::GetOverrideTargetingLifeTime();
			const EDrawDebugTrace::Type DrawDebugType = DrawTime <= 0.0f ? EDrawDebugTrace::Type::ForOneFrame : EDrawDebugTrace::Type::ForDuration;
			const FLinearColor TraceColor = FLinearColor::Red;
			const FLinearColor TraceHitColor = FLinearColor::Green;
			switch (TraceType)
			{
			case ETargetingTraceType::Sphere:
				DrawDebugSphereTraceMulti(World, StartLocation, EndLocation, GetSweptTraceRadius(TargetingHandle), DrawDebugType, bHit, Hits, TraceColor, TraceHitColor, DrawTime);
				break;
			case ETargetingTraceType::Capsule:
				DrawDebugCapsuleTraceMulti(World, StartLocation, EndLocation, GetSweptTraceRadius(TargetingHandle), GetSweptTraceCapsuleHalfHeight(TargetingHandle), OrientationQuat.Rotator(), DrawDebugType, bHit, Hits, TraceColor, TraceHitColor, DrawTime);
				break;
			case ETargetingTraceType::Box:
				DrawDebugBoxTraceMulti(World, StartLocation, EndLocation, GetSweptTraceBoxHalfExtents(TargetingHandle), OrientationQuat.Rotator(), DrawDebugType, bHit, Hits, TraceColor, TraceHitColor, DrawTime);
				break;
			default:
			case ETargetingTraceType::Line:
				DrawDebugLineTraceMulti(World, StartLocation, EndLocation, DrawDebugType, bHit, Hits, TraceColor, TraceHitColor, DrawTime);
				break;
			}
		}
	}
}

void UTargetingSelectionTask_Trace::BuildTraceResultsDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));

		for (const FTargetingDefaultResultData& TargetData : TargetResults)
		{
			if (const AActor* Target = TargetData.HitResult.GetActor())
			{
				if (ScratchPadString.IsEmpty())
				{
					ScratchPadString = FString::Printf(TEXT("%s"), *GetNameSafe(Target));
				}
				else
				{
					ScratchPadString += FString::Printf(TEXT(", %s"), *GetNameSafe(Target));
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSelectionTask_Trace::ResetTraceResultsDebugString(const FTargetingRequestHandle& TargetingHandle) const
{
#if WITH_EDITORONLY_DATA
	FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
	FString& ScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(GetNameSafe(this));
	ScratchPadString.Reset();
#endif // WITH_EDITORONLY_DATA
}

#endif // ENABLE_DRAW_DEBUG


