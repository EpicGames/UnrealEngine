// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/TargetingSortTask_Base.h"

#include "GameFramework/Actor.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Kismet/KismetMathLibrary.h"

#if ENABLE_DRAW_DEBUG
#include "Engine/Canvas.h"
#endif // ENABLE_DRAW_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingSortTask_Base)

namespace SortTaskConstants
{
	const FString PreSortPrefix = TEXT("PreSort");
	const FString PostSortPrefix = TEXT("PostSort");
}

UTargetingSortTask_Base::UTargetingSortTask_Base(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAscending = true;
	bStableSort = false;
}

float UTargetingSortTask_Base::GetScoreForTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const
{
	return 0.f;
}

void UTargetingSortTask_Base::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
	Super::Execute(TargetingHandle);

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Executing);

#if ENABLE_DRAW_DEBUG
	ResetSortDebugStrings(TargetingHandle);
#endif // ENABLE_DRAW_DEBUG

	if (TargetingHandle.IsValid())
	{
		if (FTargetingDefaultResultsSet* ResultData = FTargetingDefaultResultsSet::Find(TargetingHandle))
		{
#if ENABLE_DRAW_DEBUG
			BuildPreSortDebugString(TargetingHandle, ResultData->TargetResults);
#endif // ENABLE_DRAW_DEBUG

			const int32 NumTargets = ResultData->TargetResults.Num();

			// We get the highest score first so we can normalize the score afterwards, 
			// every task should have the same max score so none weights more than the others
			float HighestScore = 0.f;
			TArray<float> RawScores;
			for (const FTargetingDefaultResultData& TargetResult : ResultData->TargetResults)
			{
				const float RawScore = GetScoreForTarget(TargetingHandle, TargetResult);
				RawScores.Add(RawScore);
				HighestScore = FMath::Max(HighestScore, RawScore);
			}

			if(ensureMsgf(NumTargets == RawScores.Num(), TEXT("The cached raw scores should be the same size as the number of targets!")))
			{
				// Adding the normalized scores to each target result.
				for (int32 TargetIterator = 0; TargetIterator < NumTargets; ++TargetIterator)
				{
					FTargetingDefaultResultData& TargetResult = ResultData->TargetResults[TargetIterator];
					
					// Driving ascending/descending sorting based on a multiplier so it carries over to other tasks 
					float SortingMultiplier = bAscending ? 1.f : -1.f;
					TargetResult.Score += UKismetMathLibrary::SafeDivide(RawScores[TargetIterator], HighestScore) * SortingMultiplier;
				}
			}

			auto ByScore = [](const FTargetingDefaultResultData& Lhs, const FTargetingDefaultResultData& Rhs)
			{
				return Lhs.Score < Rhs.Score;
			};

			// sort the set
			if (bStableSort)
			{
				ResultData->TargetResults.StableSort(ByScore);
			}
			else
			{
				ResultData->TargetResults.Sort(ByScore);
			}

#if ENABLE_DRAW_DEBUG
			BuildPostSortDebugString(TargetingHandle, ResultData->TargetResults);
#endif // ENABLE_DRAW_DEBUG
		}
	}

	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Completed);
}


#if ENABLE_DRAW_DEBUG

void UTargetingSortTask_Base::DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const
{
#if WITH_EDITORONLY_DATA
	if (TargetingSubsystem && UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		const FString& PreSortScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(SortTaskConstants::PreSortPrefix + GetNameSafe(this));
		const FString& PostSortScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(SortTaskConstants::PostSortPrefix + GetNameSafe(this));
		if (!PreSortScratchPadString.IsEmpty() && !PostSortScratchPadString.IsEmpty())
		{
			if (Info.Canvas)
			{
				Info.Canvas->SetDrawColor(FColor::Yellow);
			}

			FString TaskString = FString::Printf(TEXT("Initial : %s"), *PreSortScratchPadString);
			TargetingSubsystem->DebugLine(Info, TaskString, XOffset, YOffset, MinTextRowsToAdvance);

			TaskString = FString::Printf(TEXT("Sorted : %s"), *PostSortScratchPadString);
			TargetingSubsystem->DebugLine(Info, TaskString, XOffset, YOffset, MinTextRowsToAdvance);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSortTask_Base::BuildPreSortDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& PreSortScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(SortTaskConstants::PreSortPrefix + GetNameSafe(this));

		for (const FTargetingDefaultResultData& TargetData : TargetResults)
		{
			if (const AActor* Target = TargetData.HitResult.GetActor())
			{
				if (PreSortScratchPadString.IsEmpty())
				{
					PreSortScratchPadString = *GetNameSafe(Target);
				}
				else
				{
					PreSortScratchPadString += FString::Printf(TEXT(", %s"), *GetNameSafe(Target));
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSortTask_Base::BuildPostSortDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const
{
#if WITH_EDITORONLY_DATA
	if (UTargetingSubsystem::IsTargetingDebugEnabled())
	{
		FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
		FString& PostSortScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(SortTaskConstants::PostSortPrefix + GetNameSafe(this));

		for (const FTargetingDefaultResultData& TargetData : TargetResults)
		{
			if (const AActor* Target = TargetData.HitResult.GetActor())
			{
				if (PostSortScratchPadString.IsEmpty())
				{
					PostSortScratchPadString = *GetNameSafe(Target);
				}
				else
				{
					PostSortScratchPadString += FString::Printf(TEXT(", %s"), *GetNameSafe(Target));
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UTargetingSortTask_Base::ResetSortDebugStrings(const FTargetingRequestHandle& TargetingHandle) const
{
#if WITH_EDITORONLY_DATA
	FTargetingDebugData& DebugData = FTargetingDebugData::FindOrAdd(TargetingHandle);
	FString& PreSortScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(SortTaskConstants::PreSortPrefix + GetNameSafe(this));
	PreSortScratchPadString.Reset();

	FString& PostSortScratchPadString = DebugData.DebugScratchPadStrings.FindOrAdd(SortTaskConstants::PostSortPrefix + GetNameSafe(this));
	PostSortScratchPadString.Reset();
#endif // WITH_EDITORONLY_DATA
}

#endif // ENABLE_DRAW_DEBUG
