// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tasks/TargetingTask.h"

#include "GameFramework/Actor.h"
#include "Types/TargetingSystemTypes.h"
#include "TargetingSystem/TargetingSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingTask)


UTargetingTask::UTargetingTask(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UTargetingTask::Init(const FTargetingRequestHandle& TargetingHandle) const
{
	SetTaskAsyncState(TargetingHandle, ETargetingTaskAsyncState::Initialized);
}

void UTargetingTask::Execute(const FTargetingRequestHandle& TargetingHandle) const
{
}

void UTargetingTask::CancelAsync() const
{
}

bool UTargetingTask::IsAsyncTargetingRequest(const FTargetingRequestHandle& TargetingHandle) const
{
	if (FTargetingAsyncTaskData* AsyncStateData = FTargetingAsyncTaskData::Find(TargetingHandle))
	{
		return AsyncStateData->bAsyncRequest;
	}

	return false;
}

void UTargetingTask::SetTaskAsyncState(const FTargetingRequestHandle& TargetingHandle, ETargetingTaskAsyncState AsyncState) const
{
	if (FTargetingAsyncTaskData* AsyncStateData = FTargetingAsyncTaskData::Find(TargetingHandle))
	{
		if (AsyncStateData->bAsyncRequest)
		{
			if (const FTargetingTaskSet** PtrToTaskSet = FTargetingTaskSet::Find(TargetingHandle))
			{
				if (const FTargetingTaskSet* TaskSet = (*PtrToTaskSet))
				{
					const int32 CurrentTaskIndex = AsyncStateData->CurrentAsyncTaskIndex;
					if (this == TaskSet->Tasks[CurrentTaskIndex])
					{
						AsyncStateData->CurrentAsyncTaskState = AsyncState;
					}
				}
			}
		}
	}
}

ETargetingTaskAsyncState UTargetingTask::GetTaskAsyncState(const FTargetingRequestHandle& TargetingHandle) const
{
	if (FTargetingAsyncTaskData* AsyncStateData = FTargetingAsyncTaskData::Find(TargetingHandle))
	{
		if (AsyncStateData->bAsyncRequest)
		{
			if (const FTargetingTaskSet** PtrToTaskSet = FTargetingTaskSet::Find(TargetingHandle))
			{
				if (const FTargetingTaskSet* TaskSet = (*PtrToTaskSet))
				{
					const int32 CurrentTaskIndex = AsyncStateData->CurrentAsyncTaskIndex;
					if (this == TaskSet->Tasks[CurrentTaskIndex])
					{
						return AsyncStateData->CurrentAsyncTaskState;
					}
				}
			}
		}
	}

	return ETargetingTaskAsyncState::Unitialized;
}

UWorld* UTargetingTask::GetSourceContextWorld(const FTargetingRequestHandle& TargetingHandle) const
{
	if (const FTargetingSourceContext* SourceContext = FTargetingSourceContext::Find(TargetingHandle))
	{
		if (SourceContext->SourceActor)
		{
			return SourceContext->SourceActor->GetWorld();
		}

		if (SourceContext->InstigatorActor)
		{
			return SourceContext->InstigatorActor->GetWorld();
		}
	}

	return GetWorld();
}

UTargetingSubsystem* UTargetingTask::GetTargetingSubsystem(const FTargetingRequestHandle& TargetingHandle) const
{
	if (const FTargetingRequestData* RequestData = FTargetingRequestData::Find(TargetingHandle))
	{
		return RequestData->TargetingSubsystem;
	}
	return nullptr;
}
