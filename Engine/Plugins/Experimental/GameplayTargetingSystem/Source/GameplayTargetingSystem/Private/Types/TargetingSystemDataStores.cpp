// Copyright Epic Games, Inc. All Rights Reserved.
#include "Types/TargetingSystemDataStores.h"

namespace UE
{
	namespace TargetingSystem
	{
		/** Data Stores used for Targeting Requests */
		TTargetingDataStore<FTargetingRequestData> GTargetingRequestDataStore;
		TTargetingDataStore<const FTargetingTaskSet*> GTargetingTaskSetDataStore;
		TTargetingDataStore<FTargetingSourceContext> GTargetingSourceDataStore;
		TTargetingDataStore<FTargetingDefaultResultsSet> GTargetingResultsDataStore;
		TTargetingDataStore<FTargetingAsyncTaskData> GTargetingAsyncTaskDataStore;
		TTargetingDataStore<FTargetingImmediateTaskData> GTargetingImmediateTaskDataStore;

#if ENABLE_DRAW_DEBUG

		TTargetingDataStore<FTargetingDebugData> GTargetingDebugDataStore;

#endif // ENABLE_DRAW_DEBUG

	} // TargetingSystem
} // UE