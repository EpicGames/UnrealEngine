// Copyright Epic Games, Inc. All Rights Reserved.
#include "Types/TargetingSystemTypes.h"

#include "TargetingSystem/TargetingPreset.h"
#include "TargetingSystem/TargetingSubsystem.h"
#include "Tasks/TargetingTask.h"
#include "Types/TargetingSystemLogs.h"
#include "Types/TargetingSystemDataStores.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TargetingSystemTypes)


/** @struct FTargetingRequestHandle */

void FTargetingRequestHandle::Reset()
{
	Handle = 0;
}

bool FTargetingRequestHandle::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	TARGETING_LOG(Fatal, TEXT("FTargetingRequestHandle::NetSerialize should not be called!"));
	return false;
}

FTargetingRequestHandle::FOnTargetingRequestHandleReleased& FTargetingRequestHandle::GetReleaseHandleDelegate()
{
	static FTargetingRequestHandle::FOnTargetingRequestHandleReleased Delegate;
	return Delegate;
}

/**	@struct FTargetingTaskSet */

const FTargetingTaskSet*& FTargetingTaskSet::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingTaskSetDataStore.FindOrAdd(Handle);
}

const FTargetingTaskSet** FTargetingTaskSet::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingTaskSetDataStore.Find(Handle);
}


/**	@struct FTargetingDefaultResultsSet */

FTargetingDefaultResultsSet& FTargetingDefaultResultsSet::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingResultsDataStore.FindOrAdd(Handle);
}

FTargetingDefaultResultsSet* FTargetingDefaultResultsSet::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingResultsDataStore.Find(Handle);
}


/**	@struct FTargetingSourceContext */

FTargetingSourceContext& FTargetingSourceContext::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingSourceDataStore.FindOrAdd(Handle);
}

FTargetingSourceContext* FTargetingSourceContext::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingSourceDataStore.Find(Handle);
}


/**	@struct FTargetingRequestData */

FTargetingRequestData& FTargetingRequestData::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingRequestDataStore.FindOrAdd(Handle);
}


FTargetingRequestData* FTargetingRequestData::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingRequestDataStore.Find(Handle);
}

void FTargetingRequestData::Initialize(FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate, UTargetingSubsystem* Subsystem)
{
	TargetingSubsystem = Subsystem;
	bComplete = false;
	TargetingRequestDelegate = CompletionDelegate;
	TargetingRequestDynamicDelegate = CompletionDynamicDelegate;
}

void FTargetingRequestData::BroadcastTargetingRequestDelegate(FTargetingRequestHandle TargetingRequestHandle)
{
	TargetingRequestDelegate.ExecuteIfBound(TargetingRequestHandle);
	TargetingRequestDynamicDelegate.ExecuteIfBound(TargetingRequestHandle);
}


/** @struct FTargetingAsyncTaskData */

FTargetingAsyncTaskData& FTargetingAsyncTaskData::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingAsyncTaskDataStore.FindOrAdd(Handle);
}

FTargetingAsyncTaskData* FTargetingAsyncTaskData::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingAsyncTaskDataStore.Find(Handle);
}

void FTargetingAsyncTaskData::InitializeForAsyncProcessing()
{
	bAsyncRequest = true;
	CurrentAsyncTaskIndex = 0;
	CurrentAsyncTaskState = ETargetingTaskAsyncState::Unitialized;
}

FTargetingImmediateTaskData& FTargetingImmediateTaskData::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingImmediateTaskDataStore.FindOrAdd(Handle);
}

FTargetingImmediateTaskData* FTargetingImmediateTaskData::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingImmediateTaskDataStore.Find(Handle);
}

#if WITH_EDITORONLY_DATA

FTargetingDebugData& FTargetingDebugData::FindOrAdd(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingDebugDataStore.FindOrAdd(Handle);
}

FTargetingDebugData* FTargetingDebugData::Find(FTargetingRequestHandle Handle)
{
	return UE::TargetingSystem::GTargetingDebugDataStore.Find(Handle);
}

#endif // WITH_EDITORONLY_DATA

