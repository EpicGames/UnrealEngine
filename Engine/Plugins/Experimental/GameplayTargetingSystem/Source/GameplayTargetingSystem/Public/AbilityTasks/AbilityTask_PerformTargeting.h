// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Types/TargetingSystemTypes.h"
#include "AbilityTask_PerformTargeting.generated.h"

#define UE_API TARGETINGSYSTEM_API

class UTargetingPreset;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTargetReadyDelegate, FTargetingRequestHandle, TargetingRequestHandle);

UCLASS(MinimalAPI)
class UAbilityTask_PerformTargeting : public UAbilityTask
{
	GENERATED_BODY()

public:
	// Called when the targeting request has been completed and results are ready
	UPROPERTY(BlueprintAssignable)
	FTargetReadyDelegate OnTargetReady;

	UE_API virtual void Activate() override;

	// Performs a targeting request based on a Targeting Preset from a GameplayAbility
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_PerformTargeting* PerformTargetingRequest(UGameplayAbility* OwningAbility, UTargetingPreset* InTargetingPreset, bool bAllowAsync);
	
	// Performs a target filtering request based on a Targeting Preset from a GameplayAbility
	UFUNCTION(BlueprintCallable, Category = "Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UE_API UAbilityTask_PerformTargeting* PerformFilteringRequest(UGameplayAbility* OwningAbility, UTargetingPreset* TargetingPreset, const TArray<AActor*> InTargets, bool bAllowAsync);

private:
	// Helper function to push the InitialTargets array to the DataStore for Filtering requests
	void SetupInitialTargetsForRequest(FTargetingRequestHandle RequestHandle) const;

protected:
	// Targeting Preset to request
	TObjectPtr<UTargetingPreset> TargetingPreset;

	// Initial Targets to pass into the request (only populated for Filtering Requests)
	TArray<AActor*> InitialTargets;

	// Whether to perform this request immediately or async
	bool bPerformAsync;
};

#undef UE_API
