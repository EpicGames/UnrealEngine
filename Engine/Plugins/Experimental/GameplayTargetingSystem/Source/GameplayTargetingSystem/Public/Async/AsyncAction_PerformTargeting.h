// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/TargetingSystemTypes.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "UObject/ObjectMacros.h"

#include "AsyncAction_PerformTargeting.generated.h"

class AActor;
class UTargetingPreset;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPerformTargetingReady, FTargetingRequestHandle, TargetingHandle);


/**
*	@class UAsyncAction_PerformTargeting
*/
UCLASS(BlueprintType, meta = (ExposedAsyncProxy = "AsyncTaskRef"))
class UAsyncAction_PerformTargeting : public UBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintCallable, meta = (DefaultToSelf = "SourceActor", BlueprintInternalUseOnly = "true", DisplayName = "Perform Targeting Async Action"))
	static UAsyncAction_PerformTargeting* PerformTargetingRequest(AActor* SourceActor, UTargetingPreset* TargetingPreset, bool bUseAsyncTargeting);

	UFUNCTION(BlueprintCallable, meta = (DefaultToSelf = "SourceActor", BlueprintInternalUseOnly = "true", DisplayName = "Perform Filtering Async Action"))
	static UAsyncAction_PerformTargeting* PerformFilteringRequest(AActor* SourceActor, UTargetingPreset* TargetingPreset, bool bUseAsyncTargeting, const TArray<AActor*> InTargets);

	UFUNCTION(BlueprintPure, Category = "Targeting")
	FTargetingRequestHandle GetTargetingHandle() const { return TargetingHandle; }

	virtual void Activate() override;

public:
	UPROPERTY(BlueprintAssignable)
	FPerformTargetingReady Targeted;

private:
	/** Method to seed the targeting request w/ the initial set of targets */
	void SetupInitialTargetsForRequest() const;

private:
	/** The targeting preset to use for targeting */
	UPROPERTY()
	TObjectPtr<UTargetingPreset> TargetingPreset = nullptr;

	/** The actor this task is running for */
	UPROPERTY()
	TWeakObjectPtr<AActor> WeakSourceActor;

	/** A set of targets to pre-seed the targeting request with (great for filtering targets without the needs for selection) */
	UPROPERTY()
	TArray<TObjectPtr<AActor>> InitialTargets;

	/** Targeting Handle created for this Async targeting task */
	UPROPERTY()
	FTargetingRequestHandle TargetingHandle;

	/** Flag to indicate we should be using async targeting (default is immediate execution) */
	UPROPERTY()
	uint8 bUseAsyncTargeting : 1;
};
