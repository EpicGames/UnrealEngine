// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Engine/DataAsset.h"
#include "Types/TargetingSystemTypes.h"

#include "TargetingPreset.generated.h"

#define UE_API TARGETINGSYSTEM_API

class UTargetingTask;
struct FTargetingRequestData;
struct FTargetingDefaultResultsSet;


/**
*	@class UTargetingPreset
*
*	This object is used when a data asset is preferred for task setup.
*
*	Tasks will be processed in the order they are setup in the task set. It
*	is recommended that selection tasks happen first before any filtering
*	and sorting.
*/
UCLASS(MinimalAPI, BlueprintType)
class UTargetingPreset : public UDataAsset
{
	GENERATED_BODY()

public:
	UE_API UTargetingPreset(const FObjectInitializer& ObjectInitializer);

	/** Get the set of tasks defined for this targeting preset */
	inline const FTargetingTaskSet* GetTargetingTaskSet() const { return &TargetingTaskSet; }
	inline FTargetingTaskSet& GetMutableTargetingTaskSet() { return TargetingTaskSet; }

	/** The tasks that make up this targeting preset */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Targeting", meta = (ShowOnlyInnerProperties))
	FTargetingTaskSet TargetingTaskSet;
};

#undef UE_API
