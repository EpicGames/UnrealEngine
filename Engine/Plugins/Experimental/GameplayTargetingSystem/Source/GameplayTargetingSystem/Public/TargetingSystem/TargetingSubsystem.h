// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/SortedMap.h"
#include "CoreMinimal.h"
#include "DrawDebugHelpers.h"
#include "Misc/CoreMisc.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "Tasks/CollisionQueryTaskData.h" 
#include "Tickable.h"
#include "Types/TargetingSystemTypes.h"

#include "TargetingSubsystem.generated.h"

class AHUD;
class FDebugDisplayInfo;
class FSubsystemCollectionBase;
class UCanvas;
class UTargetingPreset;
class UTargetingTask;
class UWorld;


#if ENABLE_DRAW_DEBUG
/**
*	@struct FTargetingDebugInfo
*/
struct FTargetingDebugInfo
{
	FTargetingDebugInfo()
	{
		FMemory::Memzero(*this);
	}

	class UCanvas* Canvas;

	bool bPrintToLog;

	float XPos;
	float YPos;
	float OriginalX;
	float OriginalY;
	float MaxY;
	float NewColumnYPadding;
	float YL;
};
#endif // ENABLE_DRAW_DEBUG

/**
*	@class UTargetingSubsystem
*
*	The Targeting Subsystem is the entry point for users to initiate targeting requests.
*
*	The entry point to the system is the target request handle. The handle is used to interface with
*	the targeting data stores. Data stores are templated classes around generic data structs that the
*	system and tasks use to accomplish a targeting request.
*
*	The targeting system has 3 mandatory data stores and 1 required for async targeting request. These
*	data stores are required to be set up before the system can properly run a targeting request. The 
*	mandatory 3 data stores are FTargetingRequestData, FTargetingTaskSet, and FTargetingSourceContext.
*	FTargetingAsyncTaskData is implicitly setup when an async targeting request is initiated.
*
*	Users can do all the pieces manually in C++ by setting up the required data stores themselves, 
*	or, to have it a bit more automated, the user can use the APIs that utilize UTargetingPreset data asset. 
*
*	For immediate targeting requests users will call the Execute methods. These functions perform all the
*	tasks till completion. The system will not go latent.
*
*	For async targeting requests users will call the Start Async methods. The system will queue up a targeting
*	request and as each task is processed the system can run through all the tasks to completion or stop processing
*	until the next frame while it waits for a task to complete.
* 
*	Note about Targeting Handles, when a targeting handle is created it will not implicitly release the handle.
*	It is up to the creator to either grab a Async Task Data or Immediate Task Data and set a flag indicating 
*	the system should do it for them after the callback fires, or it is up to the user to release the handle
*	when they are done with it.
*/
UCLASS(MinimalAPI, DisplayName = "Targeting Subsystem")
class UTargetingSubsystem : public UGameInstanceSubsystem, public FTickableGameObject, public FSelfRegisteringExec
{
	GENERATED_BODY()

public:
	/** Method to get the mesh network events subsystem */
	TARGETINGSYSTEM_API static UTargetingSubsystem* Get(const UWorld* World);
	TARGETINGSYSTEM_API static UTargetingSubsystem* GetTargetingSubsystem(const UObject* WorldContextObject);

	/** Implemented to emit references from the global data stores */
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:
	UTargetingSubsystem();

	// Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	// End USubsystem

	/** FSelfRegisteringExec implementation */
	virtual bool Exec_Runtime(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	/** ~FSelfRegisteringExec implementation */

	// FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual ETickableTickType GetTickableTickType() const override { return (HasAnyFlags(RF_ClassDefaultObject) ? ETickableTickType::Never : ETickableTickType::Conditional); }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UTargetingSubsystem, STATGROUP_Tickables); }
	virtual UWorld* GetTickableGameObjectWorld() const override { return GetWorld(); }
	// ~FTickableGameObject interface

public:
	/** Target Handle Generation Methods */

	/** Method to create a basic target request handle (nothing will be setup for any targeting requests)
		The request data store, task set data store, and source context data store will have to be setup
		manually by caller. */
	TARGETINGSYSTEM_API static FTargetingRequestHandle CreateTargetRequestHandle();

	/** Method to create and setup a target request handle given a targeting preset and source context */
	TARGETINGSYSTEM_API static FTargetingRequestHandle MakeTargetRequestHandle(const UTargetingPreset* TargetingPreset, const FTargetingSourceContext& InSourceContext);

	/** Method to release a target handle (Note: Unless flagged via AsyncTaskData or ImmediateTaskData this will not release implicitly) */
	TARGETINGSYSTEM_API static void ReleaseTargetRequestHandle(FTargetingRequestHandle& Handle);


	/** Targeting Request Methods */
public:
	/** Method to execute an immediate targeting request with a given targeting handle. */
	TARGETINGSYSTEM_API void ExecuteTargetingRequestWithHandle(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate = FTargetingRequestDelegate(), FTargetingRequestDynamicDelegate CompletionDynamicDelegate = FTargetingRequestDynamicDelegate());

	/** Method to execute an immediate targeting request based on a gameplay targeting preset.*/
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Targeting System | Instant Request")
	void ExecuteTargetingRequest(const UTargetingPreset* TargetingPreset, const FTargetingSourceContext& InSourceContext, FTargetingRequestDynamicDelegate CompletionDynamicDelegate);

	/** Method to queue an async targeting request with a given targeting handle. */
	TARGETINGSYSTEM_API void StartAsyncTargetingRequestWithHandle(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate = FTargetingRequestDelegate(), FTargetingRequestDynamicDelegate CompletionDynamicDelegate = FTargetingRequestDynamicDelegate());

	/** Method to remove an async targeting request with a given targeting handle */
	UFUNCTION(BlueprintCallable, Category = "Targeting System | Async Request")
	TARGETINGSYSTEM_API void RemoveAsyncTargetingRequestWithHandle(UPARAM(ref) FTargetingRequestHandle& TargetingHandle);

	/** Method to queue an async targeting request based on a gameplay targeting preset. */
	UFUNCTION(BlueprintCallable, Category = "Targeting System | Async Request")
	FTargetingRequestHandle StartAsyncTargetingRequest(const UTargetingPreset* TargetingPreset, const FTargetingSourceContext& InSourceContext, FTargetingRequestDynamicDelegate CompletionDynamicDelegate);

private:
	/** Internal method to handle running an immediate targeting request */
	void ExecuteTargetingRequestWithHandleInternal(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate);

	/** Internal method to hanlde queuing up an async targeting request */
	void StartAsyncTargetingRequestWithHandleInternal(FTargetingRequestHandle TargetingHandle, FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate);

	/** Internal method to process the tasks for a given targeting request */
	void ProcessTargetingRequestTasks(FTargetingRequestHandle& TargetingHandle, float& TimeLeft);

	/** Method to find the currect executing task for the given handle */
	UTargetingTask* FindCurrentExecutingTask(FTargetingRequestHandle Handle) const;

	/** Called when we set bTickingAsyncRequests to false, at this point it's safe to perform any queued operations on the Async Requests Array */
	void OnFinishedTickingAsyncRequests();

	/** Internal method to clear all async requests and release all references */
	void ClearAsyncRequests();

	/** Called when switching maps. Allows to release references that would otherwise prevent cleaning up an old world */
	void HandlePreLoadMap(const FString& MapName);


	/** The set of target requests queued up for async processing */
	UPROPERTY(Transient)
	TArray<FTargetingRequestHandle> AsyncTargetingRequests;

	/** Flag indicating the targeting system is currently in its tick processing targeting tasks for async request */
	bool bTickingAsycnRequests = false;

	/** While we're processing targeting requests, we add any incoming requests to this array to prevent memory stomps */
	TSortedMap<FTargetingRequestHandle, FTargetingRequestData> PendingTargetingRequests;

	/** (Version for Async Requests) While we're processing targeting requests, we add any incoming requests to this array to prevent memory stomps */
	TSortedMap<FTargetingRequestHandle, FTargetingRequestData> PendingAsyncTargetingRequests;

	/** ~Targeting Request Methods */

	/** Blueprint Helper Methods */
public:
	/** Returns the targeting source context for the targeting request handle */
	UFUNCTION(BlueprintCallable, Category = "Targeting System | Targeting Types")
	TARGETINGSYSTEM_API FTargetingSourceContext GetTargetingSourceContext(FTargetingRequestHandle TargetingHandle) const;

	/** Method to get the actor targets from a given targeting request handle */
	UFUNCTION(BlueprintCallable, Category = "Targeting System | Targeting Results")
	TARGETINGSYSTEM_API void GetTargetingResultsActors(FTargetingRequestHandle TargetingHandle, TArray<AActor*>& Targets) const;

	/** Helper method to get the set of hit results for a given targeting handle */
	UFUNCTION(BlueprintCallable, Category = "Targeting System | Targeting Results")
	TARGETINGSYSTEM_API void GetTargetingResults(FTargetingRequestHandle TargetingHandle, TArray<FHitResult>& OutTargets) const;

	/** Function that lets you set a data store from a certain Targeting Handle to add some Collision Query Param Overrides  */
	UFUNCTION(BlueprintCallable, Category = "Targeting System | Data Stores")
	static void OverrideCollisionQueryTaskData(FTargetingRequestHandle TargetingHandle, const FCollisionQueryTaskData& CollisionQueryDataOverride);

	/** ~Blueprint Helper Methods */

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG

public:
	/** Helper method to indicate if the debugging feature is enabled */
	TARGETINGSYSTEM_API static bool IsTargetingDebugEnabled();

	/** Helper method to overrides the life time of the shapes drawn */
	TARGETINGSYSTEM_API static float GetOverrideTargetingLifeTime();

	/** Callback to handle OnShowDebugInfo to check if the TargetingSystem has been enabled */
	static void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	/** Helper methd to write a debug line when the ShowDebug TargetingSystem is active */
	TARGETINGSYSTEM_API void DebugLine(struct FTargetingDebugInfo& Info, FString Str, float XOffset, float YOffset, int32 MinTextRowsToAdvance = 0);

private:
	void DisplayDebug(class UCanvas* Canvas, const class FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos);
	void Debug_Internal(struct FTargetingDebugInfo& Info);
	void DebugForHandle_Internal(FTargetingRequestHandle& Handle, struct FTargetingDebugInfo& Info);
	void AccumulateScreenPos(FTargetingDebugInfo& Info);

	void AddDebugTrackedImmediateTargetRequests(FTargetingRequestHandle TargetingHandle) const;
	void AddDebugTrackedAsyncTargetRequests(FTargetingRequestHandle TargetingHandle) const;

	mutable TArray<FTargetingRequestHandle> DebugTrackedImmediateTargetRequests;
	mutable int32 CurrentImmediateRequestIndex = 0;

	mutable TArray<FTargetingRequestHandle> DebugTrackedAsyncTargetRequests;
	mutable int32 CurrentAsyncRequestIndex = 0;

#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};

