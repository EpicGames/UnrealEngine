// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/TargetingSystemDataStores.h"

#include "CollisionQueryTaskData.generated.h"

/** Data Store struct used to extend collision-based targeting tasks providing extra data from outside */
USTRUCT(BlueprintType)
struct FCollisionQueryTaskData 
{
	GENERATED_BODY()
	
	FCollisionQueryTaskData(){}

	/** Any extra actors we want to ignore. Note: Given that this is a globally-managed struct, we're manually adding refs to it in UTargetingSubsystem::AddReferencedObjects */
	UPROPERTY(BlueprintReadWrite, Category =  "Collision Query Task Overrides")
	TArray<TObjectPtr<AActor>> IgnoredActors;

	/** Called by the Targeting Subsystem to manually hang onto reference to object pointers in this struct given that it's globally-owned */
	void AddStructReferencedObjects(class FReferenceCollector& Collector);
};

DECLARE_TARGETING_DATA_STORE(FCollisionQueryTaskData)