// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Tasks/TargetingFilterTask_BasicFilterTemplate.h"

#include "SimpleTargetingFilterTask.generated.h"

#define UE_API TARGETINGSYSTEM_API

struct FTargetingRequestHandle;
struct FTargetingDefaultResultData;

/**
*	@class USimpleTargetingSelectionTask
*
*	This is a blueprintable TargetingTask class made for filtering out Targets from the results list.
*	Override the ShouldRemoveTarget Blueprint Function to define the rules for this filter.
*/
UCLASS(MinimalAPI, EditInlineNew, Abstract, Blueprintable)
class USimpleTargetingFilterTask : public UTargetingFilterTask_BasicFilterTemplate
{
	GENERATED_BODY()

public:
	//~ Begin UTargetingFilterTask_BasicFilterTemplate Interface
	UE_API virtual bool ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const override;
	//~ End UTargetingFilterTask_BasicFilterTemplate Interface

	/** Returns true if a Target should be removed from the results of this TargetingRequest */
	UFUNCTION(BlueprintImplementableEvent, DisplayName=ShouldRemoveTarget)
	UE_API bool BP_ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const;

};

#undef UE_API
