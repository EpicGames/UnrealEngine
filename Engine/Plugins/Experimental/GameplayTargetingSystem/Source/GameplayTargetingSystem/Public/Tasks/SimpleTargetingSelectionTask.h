// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "TargetingTask.h"

#include "SimpleTargetingSelectionTask.generated.h"

#define UE_API TARGETINGSYSTEM_API

class UTargetingSubsystem;
struct FTargetingRequestHandle;
struct FTargetingSourceContext;

/**
*	@class USimpleTargetingSelectionTask
*
*	This is a blueprintable TargetingTask class made for adding new Targets to the results list.
*	Override the SelectTargets function and call AddTargetHitResult or AddTargetActor to select new targets.
*/
UCLASS(MinimalAPI, EditInlineNew, Abstract, Blueprintable)
class USimpleTargetingSelectionTask : public UTargetingTask
{
	GENERATED_BODY()

public:
	/** Evaluation function called by derived classes to process the targeting request */
	UE_API virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const override;
	
	UFUNCTION(BlueprintNativeEvent)
	UE_API void SelectTargets(const FTargetingRequestHandle& TargetingHandle, const FTargetingSourceContext& SourceContext) const;

	virtual void SelectTargets_Implementation(const FTargetingRequestHandle& TargetingHandle, const FTargetingSourceContext& SourceContext) const {};

protected:
	/**
	 * Adds a single Actor to the Targeting Results for a given TargetingRequestHandle.
	 * Returns false when the Actor was already in the list.
	 * 
	 * NOTE: If you have a HitResult associated with this selection, please use AddHitResult instead.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=False, Category=Targeting)
	UE_API bool AddTargetActor(const FTargetingRequestHandle& TargetingHandle, AActor* Actor) const;

	/**
	 * Adds a HitResult to the Targeting Results for a given TargetingRequestHandle.
	 * Returns False when the Actor that was hit was already in the list
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=False, Category=Targeting)
	UE_API bool AddHitResult(const FTargetingRequestHandle& TargetingHandle, const FHitResult& HitResult) const;
	
};


#undef UE_API
