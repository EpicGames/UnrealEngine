// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Tasks/TargetingSortTask_Base.h"

#include "SimpleTargetingSortTask.generated.h"

#define UE_API TARGETINGSYSTEM_API

struct FTargetingRequestHandle;

/**
*	@class USimpleTargetingSortTask
*
*	This is a blueprintable TargetingTask class made for reordering Targets in the results list.
*	Override the GetScoreForTarget Blueprint Function to provide a score for each element and then Targets will be
*	sorted by score.
*/
UCLASS(MinimalAPI, EditInlineNew, Abstract, Blueprintable)
class USimpleTargetingSortTask : public UTargetingSortTask_Base
{
	GENERATED_BODY()

protected:
	/** Called on every target to get a Score for sorting. This score will be added to the Score float in FTargetingDefaultResultData */
	UE_API virtual float GetScoreForTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const override;

	/** Called on every target to get a Score for sorting. This score will be added to the Score float in FTargetingDefaultResultData */
	UFUNCTION(BlueprintImplementableEvent, DisplayName=GetScoreForTarget, Category=Targeting)
	UE_API float BP_GetScoreForTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const;

};


#undef UE_API
