// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Tasks/TargetingFilterTask_BasicFilterTemplate.h"
#include "Types/TargetingSystemTypes.h"

#include "TargetingFilterTask_ActorClass.generated.h"


/**
*	@class UTargetingFilterTask_ActorClass
*
*	Simple filtering task where we check the targets class type against the
*	required and ignored class types.
*/
UCLASS(Blueprintable)
class UTargetingFilterTask_ActorClass : public UTargetingFilterTask_BasicFilterTemplate
{
	GENERATED_BODY()

public:
	UTargetingFilterTask_ActorClass(const FObjectInitializer& ObjectInitializer);

	/** Evaluation function called by derived classes to process the targeting request */
	virtual bool ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const override;

protected:
	/** The set of required actor classes (must be one of these types to not be filtered out) */
	UPROPERTY(EditAnywhere, Category = "Targeting Filter | Class", Meta = (AllowAbstract=true))
	TArray<TObjectPtr<UClass>> RequiredActorClassFilters;

	/** The set of ignored actor classes (must NOT be one of these types) */
	UPROPERTY(EditAnywhere, Category = "Targeting Filter | Class", Meta = (AllowAbstract = true))
	TArray<TObjectPtr<UClass>> IgnoredActorClassFilters;
};

