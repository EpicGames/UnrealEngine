// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TargetingTask.h"
#include "UObject/Object.h"

#include "TargetingFilterTask_BasicFilterTemplate.generated.h"

#define UE_API TARGETINGSYSTEM_API

class AActor;
class UTargetingSubsystem;
struct FTargetingDebugInfo;
struct FTargetingDefaultResultData;
struct FTargetingRequestHandle;


/**
*	@class UTargetingFilterTask_BasicFilterTemplate
*
*	A base class that has a basic setup struct that a majority of filtering tasks
*	will find convenient.
*/
UCLASS(MinimalAPI, Abstract)
class UTargetingFilterTask_BasicFilterTemplate : public UTargetingTask
{
	GENERATED_BODY()

public:
	UE_API UTargetingFilterTask_BasicFilterTemplate(const FObjectInitializer& ObjectInitializer);

private:
	/** Evaluation function called by derived classes to process the targeting request */
	UE_API virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const override;

protected:
	/** Called against every target data to determine if the target should be filtered out */
	UE_API virtual bool ShouldFilterTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const;

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG
private:
	UE_API virtual void DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const override;
	void AddFilteredTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const;
	void ResetFilteredTarget(const FTargetingRequestHandle& TargetingHandle) const;
#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};

#undef UE_API
