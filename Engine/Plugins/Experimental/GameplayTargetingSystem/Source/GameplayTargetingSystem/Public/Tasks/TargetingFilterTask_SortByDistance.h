// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Types/TargetingSystemTypes.h"
#include "TargetingTask.h"
#include "UObject/Object.h"
#include "TargetingSortTask_Base.h"

#include "TargetingFilterTask_SortByDistance.generated.h"

class UTargetingSubsystem;
struct FTargetingDebugInfo;
struct FTargetingDefaultResultData;
struct FTargetingRequestHandle;


/**
*	@class UTargetingFilterTask_SortByDistance
*
*	Simple sorting filter based on the distance to the source actor.
*	Note: This filter will use the FTargetingDefaultResultsSet and 
*	use the Score factor defined for each target to store the distance
*	and sort by that value.
*/
UCLASS(Blueprintable)
class UTargetingFilterTask_SortByDistance : public UTargetingSortTask_Base
{
	GENERATED_BODY()

protected:
	virtual float GetScoreForTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const override;

	/** Use the distance to the nearest blocking collision surface on each actor instead of the distance to the actor's location */
	UPROPERTY(EditAnywhere, Category = "Target Sorting | Sort By Distance Data")
	bool bUseDistanceToNearestBlockingCollider = false;

	/** 
	* Collision channel used to determine the closest point on a actor's collider. 
	* Colliders will only be considered if they block this channel
	*/
	UPROPERTY(EditAnywhere, Category = "Target Sorting | Sort By Distance Data", Meta=(EditCondition="bUseDistanceToNearestBlockingCollider "))
	TEnumAsByte<ECollisionChannel> DistanceToCollisionChannel = ECollisionChannel::ECC_WorldDynamic;

private:
	/** Returns the Source location, whether that comes from the source actor or the Source location in the Source Context */
	FVector GetSourceLocation(const FTargetingRequestHandle& TargetingHandle) const;
};
