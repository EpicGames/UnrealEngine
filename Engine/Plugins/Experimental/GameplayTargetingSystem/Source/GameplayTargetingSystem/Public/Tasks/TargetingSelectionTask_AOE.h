// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CollisionShape.h"
#include "Engine/CollisionProfile.h"
#include "Engine/EngineTypes.h"
#include "Types/TargetingSystemTypes.h"
#include "ScalableFloat.h"
#include "TargetingTask.h"
#include "UObject/Object.h"
#include "WorldCollision.h"

#include "TargetingSelectionTask_AOE.generated.h"

#define UE_API TARGETINGSYSTEM_API

class UTargetingSubsystem;
struct FCollisionQueryParams;
struct FTargetingDebugInfo;
struct FTargetingRequestHandle;


/**
*	@enum ETargetingAOEShape	
*/
UENUM(BlueprintType)
enum class ETargetingAOEShape : uint8
{
	Box,
	Cylinder,
	Sphere,
	Capsule,
	SourceComponent,
};

/**
*	@class UTargetingSelectionTask_AOE
*
*	Basic AOE based selection task. There are 4 shape types to choose from + a custom option.
*
*	Box - Defined by a given half extent
*	Cylinder - Defined by the half extent (box) + radius
*	Sphere	- Defined by a given radius
*	Capsule	- Defined by the radius + half height
* 
*	SourceComponent - Use a collision component with a specific component tag as the shape
*/
UCLASS(MinimalAPI, Blueprintable)
class UTargetingSelectionTask_AOE : public UTargetingTask
{
	GENERATED_BODY()

public:
	UE_API UTargetingSelectionTask_AOE(const FObjectInitializer& ObjectInitializer);

	/** Evaluation function called by derived classes to process the targeting request */
	UE_API virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const override;

	/** Debug draws the outlines of the set shape type. */
	UE_API void DebugDrawBoundingVolume(const FTargetingRequestHandle& TargetingHandle, const FColor& Color, const FOverlapDatum* OverlapDatum = nullptr) const;

	ETargetingAOEShape GetShapeType() const { return ShapeType; }
	UE_API void SetShapeType(ETargetingAOEShape InShapeType);
	UE_API void SetHalfExtent(FVector InHalfExtent);
	UE_API void SetRadius(FScalableFloat InRadius);
	UE_API void SetHalfHeight(FScalableFloat InHalfHeight);

	/** Native Event to get the source location for the AOE */
	UFUNCTION(BlueprintNativeEvent, Category = "Target AOE Selection")
	UE_API FVector GetSourceLocation(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get a source location offset for the AOE */
	UFUNCTION(BlueprintNativeEvent, Category = "Target AOE Selection")
	UE_API FVector GetSourceOffset(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native event to get the source rotation for the AOE  */
	UFUNCTION(BlueprintNativeEvent, Category = "Target AOE Selection")
	UE_API FQuat GetSourceRotation(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get a source rotation offset for the AOE */
	UFUNCTION(BlueprintNativeEvent, Category = "Target AOE Selection")
	UE_API FRotator GetSourceRotationOffset(const FTargetingRequestHandle& TargetingHandle) const;

	/** Helper method to build the Collision Shape */
	UE_API FCollisionShape GetCollisionShape() const;

	bool GetUseRelativeOffset() const { return bUseRelativeOffset; }

private:
	/** Method to process the trace task immediately */
	void ExecuteImmediateTrace(const FTargetingRequestHandle& TargetingHandle) const;

	/** Method to process the trace task asynchronously */
	void ExecuteAsyncTrace(const FTargetingRequestHandle& TargetingHandle) const;

	/** Callback for an async overlap */
	void HandleAsyncOverlapComplete(const FTraceHandle& InTraceHandle, FOverlapDatum& InOverlapDatum, FTargetingRequestHandle TargetingHandle) const;

	/** Method to take the overlap results and store them in the targeting result data */
	void ProcessOverlapResults(const FTargetingRequestHandle& TargetingHandle, const TArray<FOverlapResult>& Overlaps) const;
	
private:
	/** Helper method to find the custom component defined on the source actor */
	const UPrimitiveComponent* GetCollisionComponent(const FTargetingRequestHandle& TargetingHandle) const;

	/** Setup CollisionQueryParams for the AOE */
	void InitCollisionParams(const FTargetingRequestHandle& TargetingHandle, FCollisionQueryParams& OutParams) const;

protected:
	/** The shape type to use for the AOE */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	ETargetingAOEShape ShapeType = ETargetingAOEShape::Box;

	/** The collision channel to use for the overlap check (as long as Collision Profile Name is not set) */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	TEnumAsByte<ECollisionChannel> CollisionChannel;

	/** The collision profile name to use for the overlap check */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	FCollisionProfileName CollisionProfileName;

	/** The collision profile name to use for the overlap check */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	TArray<TEnumAsByte<EObjectTypeQuery>> CollisionObjectTypes;

	/** The default source location offset used by GetSourceOffset */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	FVector DefaultSourceOffset = FVector::ZeroVector;

	/** The default source rotation offset used by GetSourceRotationOffset */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	FRotator DefaultSourceRotationOffset = FRotator::ZeroRotator;

	/** Should we offset based on world or relative Source object transform? */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	uint8 bUseRelativeOffset : 1;
	
	/** Indicates the trace should ignore the source actor */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	uint8 bIgnoreSourceActor : 1;

	/** Indicates the trace should ignore the source actor */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	uint8 bIgnoreInstigatorActor : 1;

	/** When enabled, the trace will be performed against complex collision. */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Data")
	uint8 bTraceComplex : 1 = false;

	/** The half extent to use for box and cylinder
	 *	For Cylinder the X component will be the radius and Z will be its height
	 */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Box and Cylinder")
	FVector HalfExtent = FVector::ZeroVector;

	/** The radius to use for sphere and capsule overlaps */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Sphere and Capsule")
	FScalableFloat Radius = 0.0f;

	/** The half height to use for capsule overlap checks */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Capsule")
	FScalableFloat HalfHeight = 0.0f;

	/** The component tag to use if a custom component is desired as the overlap shape. 
		Use to look up the component on the source actor */
	UPROPERTY(EditAnywhere, Category = "Target AOE Selection | Custom Component")
	FName ComponentTag = NAME_None;

protected:
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG
private:
	UE_API virtual void DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const override;
	void BuildDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const;
	void ResetDebugString(const FTargetingRequestHandle& TargetingHandle) const;
#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};

#undef UE_API
