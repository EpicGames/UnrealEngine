// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Types/TargetingSystemTypes.h"
#include "TargetingTask.h"
#include "UObject/Object.h"

#include "TargetingSelectionTask_SourceActor.generated.h"

class AActor;
class UTargetingSubsystem;
struct FTargetingDebugInfo;
struct FTargetingRequestHandle;


/**
*	@class UTargetingSelectionTask_SourceActor
*
*	Simple targeting selection task that gets the source actor
*	defined in the targeting source context.
*/
UCLASS()
class UTargetingSelectionTask_SourceActor : public UTargetingTask
{
	GENERATED_BODY()

public:
	UTargetingSelectionTask_SourceActor(const FObjectInitializer& ObjectInitializer);

	/** Evaluation function called by derived classes to process the targeting request */
	virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const override;

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG
private:
	virtual void DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const override;
	void AddSourceActorDebugString(const FTargetingRequestHandle& TargetingHandle, const AActor* SourceActor) const;
	void ResetSourceActorDebugString(const FTargetingRequestHandle& TargetingHandle) const;
#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};

