// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CollisionShape.h"
#include "Engine/CollisionProfile.h"
#include "Types/TargetingSystemTypes.h"
#include "ScalableFloat.h"
#include "TargetingTask.h"
#include "UObject/Object.h"

#include "TargetingSelectionTask_Trace.generated.h"

#define UE_API TARGETINGSYSTEM_API

class UTargetingSubsystem;
struct FCollisionQueryParams;
struct FTargetingDebugInfo;
struct FTargetingDefaultResultData;
struct FTargetingRequestHandle;
struct FTraceDatum;
struct FTraceHandle;

/**
*	@enum ETargetingTraceType determines the shape of the trace used.
*/
UENUM()
enum class ETargetingTraceType : uint8
{
	Line,
	Sphere,
	/* With zero rotation, the capsule length runs along the trace's up vector (the cross product of the trace's forward vector and the trace's forward vector flattened onto the XY plane and rotated 90 degrees around the Z vector). */
	Capsule,
	Box,
};

/**
*	@class UTargetingSelectionTask_Trace
*
*	Selection task that can perform a synchronous or asynchronous trace (line/sweep)
*	to find all targets up to the first blocking hit (or its end point).
*/
UCLASS(MinimalAPI, Blueprintable)
class UTargetingSelectionTask_Trace : public UTargetingTask
{
	GENERATED_BODY()

public:
	UE_API UTargetingSelectionTask_Trace(const FObjectInitializer& ObjectInitializer);

	/** Evaluation function called by derived classes to process the targeting request */
	UE_API virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const override;

protected:
	/** Native Event to get the source location for the Trace */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API FVector GetSourceLocation(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get a source location offset for the Trace */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API FVector GetSourceOffset(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get the direction for the Trace */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API FVector GetTraceDirection(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get the length for the Trace */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API float GetTraceLength(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get the swept trace radius (only called if TraceType = ETargetingTraceType::Sphere or TraceType = ETargetingTraceType::Capsule) */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API float GetSweptTraceRadius(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get the swept trace capsule's half height (only called if  TraceType = ETargetingTraceType::Capsule) */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API float GetSweptTraceCapsuleHalfHeight(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get the swept box trace half extents (only called if TraceType = ETargetingTraceType::Box) */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API FVector GetSweptTraceBoxHalfExtents(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get the swept trace rotation relative to trace direction
	(only called if TraceType = ETargetingTraceType::Capsule or TraceType = ETargetingTraceType::Box) */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API FRotator GetSweptTraceRotation(const FTargetingRequestHandle& TargetingHandle) const;

	/** Native Event to get additional actors the Trace should ignore */
	UFUNCTION(BlueprintNativeEvent, Category = "Target Trace Selection")
	UE_API void GetAdditionalActorsToIgnore(const FTargetingRequestHandle& TargetingHandle, TArray<AActor*>& OutAdditionalActorsToIgnore) const;

private:
	/** Method to process the trace task immediately */
	void ExecuteImmediateTrace(const FTargetingRequestHandle& TargetingHandle) const;

	/** Method to process the trace task asynchronously */
	void ExecuteAsyncTrace(const FTargetingRequestHandle& TargetingHandle) const;

	/** Callback for an async trace */
	void HandleAsyncTraceComplete(const FTraceHandle& InTraceHandle, FTraceDatum& InTraceDatum, FTargetingRequestHandle TargetingHandle) const;

	/** Method to take the hit results and store them in the targeting result data */
	void ProcessHitResults(const FTargetingRequestHandle& TargetingHandle, const TArray<FHitResult>& Hits) const;

	/** Setup CollisionQueryParams for the trace */
	void InitCollisionParams(const FTargetingRequestHandle& TargetingHandle, FCollisionQueryParams& OutParams) const;

	/** For non-sphere shape traces, calculates the world rotation for that trace. */
	FQuat GetSweptTraceQuat(const FVector& TraceDirection, const FTargetingRequestHandle& TargetingHandle) const;

protected:
	/** The trace type (i.e. shape) to use */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Collision Data")
	ETargetingTraceType TraceType = ETargetingTraceType::Line;

	/** The trace channel to use */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Collision Data")
	TEnumAsByte<ETraceTypeQuery> TraceChannel;

	/** The collision profile name to use instead of trace channel (does not work for async traces) */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Collision Data")
	FCollisionProfileName CollisionProfileName;

	/** The default swept trace radius used by GetSweptTraceRadius when the trace type is set to Sphere or Capsule */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Swept Data")
	FScalableFloat DefaultSweptTraceRadius = 10.0f;
	
	/** The default swept trace radius used by GetSweptTraceRadius when the trace type is set to Capsule */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Swept Data")
	FScalableFloat DefaultSweptTraceCapsuleHalfHeight = 10.0f;

	/** The default swept extents used by GetSweptTraceExtents when the trace type is set to Box */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Swept Data")
	FScalableFloat DefaultSweptTraceBoxHalfExtentX = 10.0f;

	/** The default swept extents used by GetSweptTraceBoxHalfExtents when the trace type is set to Box */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Swept Data")
	FScalableFloat DefaultSweptTraceBoxHalfExtentY = 10.0f;

	/** The default swept extents used by GetSweptTraceBoxHalfExtents when the trace type is set to Box  */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Swept Data")
	FScalableFloat DefaultSweptTraceBoxHalfExtentZ = 10.0f;

	/** The default swept rotation (relative to the trace direction) used by GetSweptTraceRotation when the trace type is set to Capsule or Box */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Swept Data")
	FRotator DefaultSweptTraceRotation = FRotator::ZeroRotator;

	/** The default trace length to use if GetTraceLength is not overridden by a child */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	FScalableFloat DefaultTraceLength = 10.0f;

	/** The default source location offset used by GetSourceOffset */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	FVector DefaultSourceOffset = FVector::ZeroVector;

	/** An explicit trace direction to use (default uses pawn control rotation or actor forward vector in GetTraceDirection) */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	FVector ExplicitTraceDirection = FVector::ZeroVector;

	/** Indicates the trace should perform a complex trace */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	uint8 bComplexTrace : 1;

	/** Indicates whether the trace should be a multi trace or a single trace */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	uint8 bMultiTrace : 1;

	/** Indicates the trace should ignore the source actor */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	uint8 bIgnoreSourceActor : 1;

	/** Indicates the trace should ignore the source actor */
	UPROPERTY(EditAnywhere, Category = "Target Trace Selection | Trace Data")
	uint8 bIgnoreInstigatorActor : 1;

protected:
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG
private:
	UE_API virtual void DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const override;
	
	/** Draw debug info showing the results of the shape trace used for targeting. */
	UE_API virtual void DrawDebugTrace(const FTargetingRequestHandle TargetingHandle, const FVector& StartLocation, const FVector& EndLocation, const FQuat& OrientationQuat, const bool bHit, const TArray<FHitResult>& Hits) const;
	void BuildTraceResultsDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const;
	void ResetTraceResultsDebugString(const FTargetingRequestHandle& TargetingHandle) const;
#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};

#undef UE_API
