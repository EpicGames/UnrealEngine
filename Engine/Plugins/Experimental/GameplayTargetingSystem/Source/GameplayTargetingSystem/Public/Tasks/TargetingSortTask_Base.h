// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TargetingTask.h"

#include "TargetingSortTask_Base.generated.h"

#define UE_API TARGETINGSYSTEM_API

/** 
*	@class TargetingSortTask_Base
*
*	A base class that has the basic setup for a sort task.
*/
UCLASS(MinimalAPI, Abstract)
class UTargetingSortTask_Base : public UTargetingTask
{
    GENERATED_BODY()

public:
	UE_API UTargetingSortTask_Base(const FObjectInitializer& ObjectInitializer);

protected:
	/** Called on every target to get a Score for sorting. This score will be added to the Score float in FTargetingDefaultResultData */
	UE_API virtual float GetScoreForTarget(const FTargetingRequestHandle& TargetingHandle, const FTargetingDefaultResultData& TargetData) const;

private:
	/** Evaluation function called by derived classes to process the targeting request */
	UE_API virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const override;

protected:
	UPROPERTY(EditAnywhere, Category = "Target Sorting | Data")
	uint8 bAscending : 1;

	/** Should this task use a (slightly slower) sorting algorithm that preserves the relative ordering of targets with equal scores? */
	UPROPERTY(EditAnywhere, Category = "Target Sorting | Data")
	uint8 bStableSort : 1;

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG
private:
	UE_API virtual void DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance) const override;
	void BuildPreSortDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const;
	void BuildPostSortDebugString(const FTargetingRequestHandle& TargetingHandle, const TArray<FTargetingDefaultResultData>& TargetResults) const;
	void ResetSortDebugStrings(const FTargetingRequestHandle& TargetingHandle) const;
#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};

#undef UE_API
