// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DrawDebugHelpers.h"
#include "Types/TargetingSystemTypes.h"
#include "UObject/Object.h"

#include "TargetingTask.generated.h"

#define UE_API TARGETINGSYSTEM_API

class UTargetingSubsystem;
struct FTargetingRequestHandle;

#if ENABLE_DRAW_DEBUG
struct FTargetingDebugInfo;
#endif // ENABLE_DRAW_DEBUG

/**
*	@class UTargetingTask
*
*	The base object type that all Targeting Tasks will derive from. The idea
*	is the targeting system will take target requests that are collections of
*	target tasks that will potentially generate/remove and perform operations
*	on sets of targeting results data.
*
*	Potential Task Implementations:
*
*	Selection Tasks:
*	Target selection tasks would be used to build up a collection of target
*	request results. It is recommend they are added first in the targeting
*	request. Things like ray casts, AOE shapes, actors under a reticle, etc
*	are cases that generally fall under selection.
*
*	Filtering Tasks:
*	Target filtering tasks are used to reduce the target result data set to
*	those targets that match a given criteria. Things like actor class, team
*	distance, facing, etc.
*
*	Sorting Tasks:
*	Target sorting tasks would be useful to take the set and put them in an order
*	the end user might prefer to make decisions on. Distance (min/max), score rating
*	etc.
*/
UCLASS(MinimalAPI, EditInlineNew, Abstract, Const, meta=(ShowWorldContextPin="true"))
class UTargetingTask : public UObject
{
	GENERATED_BODY()

public:
	UE_API UTargetingTask(const FObjectInitializer& ObjectInitializer);

	/** Lifecycle function called when the task first begins */
	UE_API virtual void Init(const FTargetingRequestHandle& TargetingHandle) const;

	/** Evaluation function called by derived classes to process the targeting request */
	UE_API virtual void Execute(const FTargetingRequestHandle& TargetingHandle) const;

	/** Lifecycle function called when the task was cancelled while in the Executing Async state */
	UE_API virtual void CancelAsync() const;

protected:
	/** Helper method to check if this task is running in an async targeting request */
	UE_API bool IsAsyncTargetingRequest(const FTargetingRequestHandle& TargetingHandle) const;

	/** Helper method to set the async state for the task (as long as it is the currently running one) */
	UE_API void SetTaskAsyncState(const FTargetingRequestHandle& TargetingHandle, ETargetingTaskAsyncState AsyncState) const;

	/** Helper method to check if a task is currently executing an async operation */
	UE_API ETargetingTaskAsyncState GetTaskAsyncState(const FTargetingRequestHandle& TargetingHandle) const;

	/** Helper method to get the world from the source context (if possible, returns nullptr if one cannot be found) */
	UE_API UWorld* GetSourceContextWorld(const FTargetingRequestHandle& TargetingHandle) const;

	/** Helper method to get the Targeting Subsystem in TargetingTask Blueprint Types */
	UFUNCTION(BlueprintPure, Category="Targeting")
	UE_API UTargetingSubsystem* GetTargetingSubsystem(const FTargetingRequestHandle& TargetingHandle) const;

	/** Debug Helper Methods */
#if ENABLE_DRAW_DEBUG
public:
	virtual void DrawDebug(UTargetingSubsystem* TargetingSubsystem, FTargetingDebugInfo& Info, const FTargetingRequestHandle& TargetingHandle, float XOffset, float YOffset, int32 MinTextRowsToAdvance = 0) const { }
#endif // ENABLE_DRAW_DEBUG
	/** ~Debug Helper Methods */
};


#undef UE_API
