// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/SortedMap.h"
#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Engine/EngineTypes.h"
#include "TargetingSystemTypes.h"


namespace UE
{
	namespace TargetingSystem
	{
		/**
		*	@struct TTargetingDataStore
		*
		*	Templated struct to define targeting data stores. The goal is to provide a flexible
		*	way for targeting tasks to add/remove/update generic sets of data they wish to work
		*	with.
		*
		*	This allows future growth in games to write tasks either new or extending off base
		*	framework archetypes to achieve the targeting goals they need.
		*/
		template<typename Type>
		struct TTargetingDataStore
		{
			TTargetingDataStore()
			{
				ReleaseDelegateHandle = FTargetingRequestHandle::GetReleaseHandleDelegate().AddStatic(&TTargetingDataStore::OnTargetingRequestHandleReleased);
			}

			~TTargetingDataStore()
			{
				if (ReleaseDelegateHandle.IsValid())
				{
					FTargetingRequestHandle::GetReleaseHandleDelegate().Remove(ReleaseDelegateHandle);
				}
			}

			static Type& FindOrAdd(FTargetingRequestHandle Handle);
			static Type* Find(FTargetingRequestHandle Handle);
			static void OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle);

		private:
			// @note: SortMap can be a concern if the Type is a large struct and/or there is high volume of
			// use at any one time. We could explore other storage/allocation strategies as performance data
			// dictates.

			/** The Items by handle stored for targeting requests */
			TSortedMap<FTargetingRequestHandle, Type> Items;

			/** Stores the register handle to the release delegate */
			FDelegateHandle ReleaseDelegateHandle;
		};


		/** Data Stores used for Targeting Requests */

		/** FTargetingRequestData Data Store */
		extern TTargetingDataStore<FTargetingRequestData> GTargetingRequestDataStore;

		template<>
		inline FTargetingRequestData& TTargetingDataStore<FTargetingRequestData>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingRequestDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline FTargetingRequestData* TTargetingDataStore<FTargetingRequestData>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingRequestDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<FTargetingRequestData>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingRequestDataStore.Items.Remove(Handle);
		}


		/** FTargetingTaskSet Data Store */
		extern TTargetingDataStore<const FTargetingTaskSet*> GTargetingTaskSetDataStore;

		template<>
		inline const FTargetingTaskSet*& TTargetingDataStore<const FTargetingTaskSet*>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingTaskSetDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline const FTargetingTaskSet** TTargetingDataStore<const FTargetingTaskSet*>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingTaskSetDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<const FTargetingTaskSet*>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingTaskSetDataStore.Items.Remove(Handle);
		}


		/** FTargetingSourceContext Data Store */
		extern TTargetingDataStore<FTargetingSourceContext> GTargetingSourceDataStore;

		template<>
		inline FTargetingSourceContext& TTargetingDataStore<FTargetingSourceContext>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingSourceDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline FTargetingSourceContext* TTargetingDataStore<FTargetingSourceContext>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingSourceDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<FTargetingSourceContext>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingSourceDataStore.Items.Remove(Handle);
		}


		/** FTargetingDefaultResultsSet Data Store */
		extern TTargetingDataStore<FTargetingDefaultResultsSet> GTargetingResultsDataStore;

		template<>
		inline FTargetingDefaultResultsSet& TTargetingDataStore<FTargetingDefaultResultsSet>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingResultsDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline FTargetingDefaultResultsSet* TTargetingDataStore<FTargetingDefaultResultsSet>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingResultsDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<FTargetingDefaultResultsSet>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingResultsDataStore.Items.Remove(Handle);
		}


		/** FTargetingAsyncTaskData Data Store */
		extern TTargetingDataStore<FTargetingAsyncTaskData> GTargetingAsyncTaskDataStore;

		template<>
		inline FTargetingAsyncTaskData& TTargetingDataStore<FTargetingAsyncTaskData>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingAsyncTaskDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline FTargetingAsyncTaskData* TTargetingDataStore<FTargetingAsyncTaskData>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingAsyncTaskDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<FTargetingAsyncTaskData>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingAsyncTaskDataStore.Items.Remove(Handle);
		}

		/** FTargetingImmediateTaskData Data Store */
		extern TTargetingDataStore<FTargetingImmediateTaskData> GTargetingImmediateTaskDataStore;

		template<>
		inline FTargetingImmediateTaskData& TTargetingDataStore<FTargetingImmediateTaskData>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingImmediateTaskDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline FTargetingImmediateTaskData* TTargetingDataStore<FTargetingImmediateTaskData>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingImmediateTaskDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<FTargetingImmediateTaskData>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingImmediateTaskDataStore.Items.Remove(Handle);
		}

#if ENABLE_DRAW_DEBUG

		/** FTargetingDefaultResults Data Store */
		extern TTargetingDataStore<FTargetingDebugData> GTargetingDebugDataStore;

		template<>
		inline FTargetingDebugData& TTargetingDataStore<FTargetingDebugData>::FindOrAdd(FTargetingRequestHandle Handle)
		{
			return GTargetingDebugDataStore.Items.FindOrAdd(Handle);
		}

		template<>
		inline FTargetingDebugData* TTargetingDataStore<FTargetingDebugData>::Find(FTargetingRequestHandle Handle)
		{
			return GTargetingDebugDataStore.Items.Find(Handle);
		}

		template<>
		inline void TTargetingDataStore<FTargetingDebugData>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)
		{
			GTargetingDebugDataStore.Items.Remove(Handle);
		}

#endif // ENABLE_DRAW_DEBUG

	} // TargetingSystem
} // UE

/** 
 * Helper macros for creating custom data stores for the targeting system
 * These data stores can be used to track information linked to a specific
 * Targeting Request in TargetingTasks.
 * 
 * To create a custom data store, place a DECLARE_TARGETING_DATA_STORE
 * in a header file and a DEFINE_TARGETING_DATA_STORE in a source file,
 * using the data type to be stored as the macro parameter.
 * 
 * For example: A data store that holds a struct FCustomTargetingData
 * would be declared as follows
 * 
 * In CustomTargetingData.h:
 * DECLARE_TARGETING_DATA_STORE(FCustomTargetingData)
 * 
 * In CustomTargetingData.cpp
 * DEFINE_TARGETING_DATA_STORE(FCustomTargetingData)
 * 
 * This new Data Store could be accessed in a relevant TargetingTask like
 * TTargetingDataStore<FCustomTargetingData>::FindOrAdd(Handle)
 */ 
#define DECLARE_TARGETING_DATA_STORE(DataType) namespace UE																	\
{																															\
	namespace TargetingSystem																								\
	{																														\
		extern TTargetingDataStore<DataType> GTargetingDataStore##DataType;													\
																															\
		template<>																											\
		inline DataType& TTargetingDataStore<DataType>::FindOrAdd(FTargetingRequestHandle Handle)						\
		{																													\
			return GTargetingDataStore##DataType.Items.FindOrAdd(Handle);													\
		}																													\
																															\
		template<>																											\
		inline DataType* TTargetingDataStore<DataType>::Find(FTargetingRequestHandle Handle)							\
		{																													\
			return GTargetingDataStore##DataType.Items.Find(Handle);														\
		}																													\
																															\
		template<>																											\
		inline void TTargetingDataStore<DataType>::OnTargetingRequestHandleReleased(FTargetingRequestHandle Handle)	\
		{																													\
			GTargetingDataStore##DataType.Items.Remove(Handle);																\
		}																													\
	}																														\
}

#define DEFINE_TARGETING_DATA_STORE(DataType) namespace UE																	\
{																															\
	namespace TargetingSystem																								\
	{																														\
		TTargetingDataStore<DataType> GTargetingDataStore##DataType;														\
	}																														\
}
