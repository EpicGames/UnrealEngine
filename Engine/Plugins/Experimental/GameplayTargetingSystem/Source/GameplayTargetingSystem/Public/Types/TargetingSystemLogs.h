// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("TargetingSystem"), STATGROUP_TargetingSystem, STATCAT_Advanced);

DECLARE_LOG_CATEGORY_EXTERN(LogTargetingSystem, Log, All);

#define TARGETING_LOG(Verbosity, Format, ...) UE_LOG(LogTargetingSystem, Verbosity, Format, ##__VA_ARGS__)
