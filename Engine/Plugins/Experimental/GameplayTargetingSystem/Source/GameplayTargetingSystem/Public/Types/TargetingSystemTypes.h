// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DrawDebugHelpers.h"
#include "Engine/EngineTypes.h"
#include "Engine/HitResult.h"
#include "Templates/SharedPointer.h"
#include "Templates/SharedPointerInternals.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "TargetingSystemTypes.generated.h"

class AActor;
class FArchive;
class UTargetingPreset;
class UTargetingTask;
class UPackageMap;
class UTargetingSubsystem;


/**
*	@struct FTargetingRequestHandle
*
*	The handle that is created when a user wants to make a targeting request.
*/
USTRUCT(BlueprintType)
struct FTargetingRequestHandle
{
	GENERATED_BODY()

public:
	FTargetingRequestHandle()
	: Handle(0)
	{ }

	FTargetingRequestHandle(const int32 InHandle)
	: Handle(InHandle)
	{ }

	uint32 Handle;

	/** Checks handle validity */
	inline bool IsValid() const { return (Handle != 0); }

	/** Method to reset the handle */
	TARGETINGSYSTEM_API void Reset();

	/** We override NetSeralize because we should never be network serializing this handle */
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** The handle released delegate that fires right before a handle is release so all data stores can clean up their state */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTargetingRequestHandleReleased, FTargetingRequestHandle)
	TARGETINGSYSTEM_API static FOnTargetingRequestHandleReleased& GetReleaseHandleDelegate();
	
	/** overloaded operators */
	inline bool operator==(const FTargetingRequestHandle& InHandle) const { return Handle == InHandle.Handle; }
	inline bool operator!=(const FTargetingRequestHandle& InHandle) const { return !this->operator==(InHandle); }
	friend bool operator<(const FTargetingRequestHandle& Lhs, const FTargetingRequestHandle& Rhs)
	{
		return Lhs.Handle < Rhs.Handle;
	}
};

template<>
struct TStructOpsTypeTraits<FTargetingRequestHandle> : public TStructOpsTypeTraitsBase2<FTargetingRequestHandle>
{
	enum
	{
		WithCopy = true,
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
	};
};


/**
*	@struct FTargetingTaskSet
*
*	A set of tasks to be used by targeting requests to find/processes targets
*/
USTRUCT(MinimalAPI, BlueprintType)
struct FTargetingTaskSet
{
	GENERATED_BODY()

public:
	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static const FTargetingTaskSet*& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static const FTargetingTaskSet** Find(FTargetingRequestHandle Handle);

	/** The set of tasks that will be used to satisfy a targeting request  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category="Targeting Task Set")
	TArray<TObjectPtr<UTargetingTask>> Tasks;
};


/**
*	@struct FTargetingDefaultResultData
*/
USTRUCT(BlueprintType)
struct FTargetingDefaultResultData
{
	GENERATED_BODY()

public:
	FTargetingDefaultResultData()
	{ }

	/** The hit result for this target */
	UPROPERTY(BlueprintReadOnly, Category=Targeting)
	FHitResult HitResult;

	/** The score associated w/ this target */
	UPROPERTY(BlueprintReadOnly, Category=Targeting)
	float Score = 0.0f;
};

/**
*	@struct FTargetingDefaultResultsSet
*
*	The base targeting result data used by the tasks implemented at the framework level.
*	Provides an array of data (hit result / score) that tasks can add/remove/sort/etc
*	to complete a targeting request.
*/
USTRUCT()
struct FTargetingDefaultResultsSet
{
	GENERATED_BODY()

public:
	FTargetingDefaultResultsSet()
	{ }

	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static FTargetingDefaultResultsSet& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static FTargetingDefaultResultsSet* Find(FTargetingRequestHandle Handle);

	/** The hit result for this target */
	UPROPERTY()
	TArray<FTargetingDefaultResultData> TargetResults;
};


/**
*	@struct FTargetingSourceContext
*
*	Stores context information about a targeting request.
*/
USTRUCT(BlueprintType)
struct FTargetingSourceContext
{
	GENERATED_BODY()

public:
	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static FTargetingSourceContext& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static FTargetingSourceContext* Find(FTargetingRequestHandle Handle);

	/** The optional actor the targeting request sources from (i.e. player/projectile/etc) */
	UPROPERTY(BlueprintReadWrite, Category = "Targeting Source Context")
	TObjectPtr<AActor> SourceActor = nullptr;

	/** The optional instigator the targeting request is owned by (i.e. owner of a projectile) */
	UPROPERTY(BlueprintReadWrite, Category = "Targeting Source Context")
	TObjectPtr<AActor> InstigatorActor = nullptr;

	/** The optional location the targeting request will source from (i.e. do AOE targeting at x/y/z location) */
	UPROPERTY(BlueprintReadWrite, Category = "Targeting Source Context")
	FVector SourceLocation = FVector::ZeroVector;

	/** The optional socket name to use on the source actor (if an actor is defined) */
	UPROPERTY(BlueprintReadWrite, Category = "Targeting Source Context")
	FName SourceSocketName = NAME_None;

	/** The optional reference to a source uobject to use in the context */
	UPROPERTY(BlueprintReadWrite, Category = "Targeting Source Context")
	TObjectPtr<UObject> SourceObject = nullptr;
};


/** Delegate used by async targeting requests */
DECLARE_DELEGATE_OneParam(FTargetingRequestDelegate, FTargetingRequestHandle /*TargetingRequestHandle*/);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTargetingRequestDynamicDelegate, FTargetingRequestHandle, TargetingRequestHandle);

/**
*	@struct FTargetingRequestData
*
*	General purpose targeting request data. Used for general knowledge about
*	the state of the request.
*/
USTRUCT()
struct FTargetingRequestData
{
	GENERATED_BODY()

public:
	FTargetingRequestData()
	: bComplete(false)
	{ }

	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static FTargetingRequestData& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static FTargetingRequestData* Find(FTargetingRequestHandle Handle);

	/** Initializes the targeting request data for async processing */
	void Initialize(FTargetingRequestDelegate CompletionDelegate, FTargetingRequestDynamicDelegate CompletionDynamicDelegate, UTargetingSubsystem* Subsystem);

	/** Broadcasts the targeting request delegate */
	void BroadcastTargetingRequestDelegate(FTargetingRequestHandle TargetingRequestHandle);

	/** Ref to the TargetingSubsystem object processing this request */
	UTargetingSubsystem* TargetingSubsystem;

	/** Indicates this handle has completed all the targeting request */
	uint8 bComplete : 1;

	/***/
	FTargetingRequestDelegate TargetingRequestDelegate;

	/** The dynamic delegate (BP) that fires when the targeting request is complete */
	UPROPERTY()
	FTargetingRequestDynamicDelegate TargetingRequestDynamicDelegate;
};


/** @enum ETargetingTaskAsyncState */
enum class ETargetingTaskAsyncState : uint8
{
	Unitialized,	// indicates this task hasn't been started yet
	Initialized,	// indicates this task has been init'ed and is ready to execute
	Executing,		// indicates this task is currently processing targets
	Completed,		// indicates this task is finished processing targets
};

/**
*	@struct FTargetingAsyncTaskData
*
*	The set of task book keeping data for async targeting requests.
*/
USTRUCT()
struct FTargetingAsyncTaskData
{
	GENERATED_BODY()

public:
	FTargetingAsyncTaskData()
	: bAsyncRequest(false)
	, bReleaseOnCompletion(false)
	, bRequeueOnCompletion(false)
	{ }

	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static FTargetingAsyncTaskData& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static FTargetingAsyncTaskData* Find(FTargetingRequestHandle Handle);

	/** Initializes the targeting request data for async processing */
	void InitializeForAsyncProcessing();

	/** The current task index being processed for the targeting request */
	int32 CurrentAsyncTaskIndex = INDEX_NONE;

	/** The current task's async state */
	ETargetingTaskAsyncState CurrentAsyncTaskState = ETargetingTaskAsyncState::Unitialized;

	/** Indicates the targeting request was an asynchronous request */
	uint8 bAsyncRequest : 1;

	/** Indicates the targeting request should be released after completion */
	uint8 bReleaseOnCompletion : 1;

	/** Indicates the target request should be re-queued after completion */
	uint8 bRequeueOnCompletion : 1;
};

/**
*	@struct FTargetingImmediateTaskData
*
*	The set of task book keeping data for immediate targeting requests.
*/
USTRUCT()
struct FTargetingImmediateTaskData
{
	GENERATED_BODY()

public:
	FTargetingImmediateTaskData()
	: bReleaseOnCompletion(false)
	{ }

	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static FTargetingImmediateTaskData& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static FTargetingImmediateTaskData* Find(FTargetingRequestHandle Handle);

	/** Indicates the targeting request was an asynchronous request */
	uint8 bReleaseOnCompletion : 1;
};



/**
*	@struct FTargetingDebugData
*
*	The set of task book keeping data for async targeting requests.
*/
USTRUCT()
struct FTargetingDebugData
{
	GENERATED_BODY()

public:
	FTargetingDebugData()
	{ }

#if ENABLE_DRAW_DEBUG
	/** Convenience method to make using the global data store easier */
	TARGETINGSYSTEM_API static FTargetingDebugData& FindOrAdd(FTargetingRequestHandle Handle);
	TARGETINGSYSTEM_API static FTargetingDebugData* Find(FTargetingRequestHandle Handle);
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<const UTargetingPreset> TargetingPreset = nullptr;

	UPROPERTY()
	TMap<FString, FString> DebugScratchPadStrings;

	UPROPERTY()
	TArray<FTargetingDefaultResultData> CachedTargetResults;
#endif // WITH_EDITORONLY_DATA
};
