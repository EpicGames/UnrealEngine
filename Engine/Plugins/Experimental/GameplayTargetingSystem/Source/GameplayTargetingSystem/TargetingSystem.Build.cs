// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TargetingSystem : ModuleRules
{
	// TODO: Rename parent directory to TargetingSystem so it matches module name
	public TargetingSystem(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"GameplayTasks",
				"GameplayAbilities",
				"GameplayTags",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DeveloperSettings"
			}
		);
	}
}