{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Targeting System",
	"Description": "Generic targeting system for use with gameplay abilities, aim assist, etc",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "TargetingSystem",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayAbilities",
			"Enabled": true
		}
	]
}